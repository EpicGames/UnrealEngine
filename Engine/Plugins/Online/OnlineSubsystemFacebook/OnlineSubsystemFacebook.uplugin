{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem Facebook",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Facebook platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

    "Modules": [
        {
            "Name": "OnlineSubsystemFacebook",
            "Type": "Runtime",
            "LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"IOS",
				"Android",
				"Win64"
			]
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemFacebook",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		}
	]

}