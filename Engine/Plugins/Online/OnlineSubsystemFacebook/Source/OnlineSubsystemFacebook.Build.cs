// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

public class OnlineSubsystemFacebook : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Engine, "OnlineSubsystemFacebook")]
	bool bUseClassicLogin = true;

	protected virtual bool bUsesRestfulImpl
	{
		get =>
			Target.Platform == UnrealTargetPlatform.Win64 ||
			Target.Platform == UnrealTargetPlatform.Mac ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
	}

	public OnlineSubsystemFacebook(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDefinitions.Add("ONLINESUBSYSTEMFACEBOOK_PACKAGE=1");
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreOnline",
				"CoreUObject",
				"ApplicationCore",
				"HTTP",
				"ImageCore",
				"Json",
				"OnlineSubsystem", 
			}
			);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Facebook");

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			bEnableObjCAutomaticReferenceCounting = true;
			PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;

			PublicDefinitions.Add("WITH_FACEBOOK=1");
			PrivateIncludePaths.Add("Private/IOS");
			
			ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);

			if (bUseClassicLogin)
			{
				// Do not use or include AppTrackingTransparency framework if not needed
				PublicDefinitions.Add("UE_WITH_CLASSIC_FACEBOOK_LOGIN=1");
				PublicFrameworks.Add("AppTrackingTransparency");
			}
			else
			{
				PublicDefinitions.Add("UE_WITH_CLASSIC_FACEBOOK_LOGIN=0");
			}

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("IOSPlugin", Path.Combine(PluginPath, "OnlineSubsystemFacebookIOS_UPL.xml"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PublicDefinitions.Add("WITH_FACEBOOK=1");
			PrivateIncludePaths.Add("Private/Android");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Launch",
				}
			);

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "OnlineSubsystemFacebookAndroid_UPL.xml"));
		}
		else if (bUsesRestfulImpl)
		{
			PublicDefinitions.Add("WITH_FACEBOOK=1");
			PrivateIncludePaths.Add("Private/Rest");
		}
        else
        {
			PublicDefinitions.Add("WITH_FACEBOOK=0");

			PrecompileForTargets = PrecompileTargetsType.None;
		}

		PublicDefinitions.Add("USES_RESTFUL_FACEBOOK=" + (bUsesRestfulImpl ? "1" : "0"));
	}
}
