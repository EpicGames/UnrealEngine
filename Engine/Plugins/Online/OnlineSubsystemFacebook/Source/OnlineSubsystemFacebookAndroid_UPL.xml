<?xml version="1.0" encoding="utf-8"?>
<!--OnlineSubsystemFacebook plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Facebook SDK Android init"/>
		<log text="Facebook requires minimum SDK of 15"/>
		
		<!-- get packaging for Facebook from ini and reset it if architecture not supported -->
		<setBoolFromProperty result="bEnabled" ini="Engine" section="OnlineSubsystemFacebook" property="bEnabled" default="false"/>
		<setStringFromProperty result="ClientId" ini="Engine" section="OnlineSubsystemFacebook" property="ClientId" default="0"/>
		<setStringFromProperty result="ClientToken" ini="Engine" section="OnlineSubsystemFacebook" property="ClientToken" default="0"/>
		<setBoolFromProperty result="bEnableAutomaticLogging" ini="Engine" section="OnlineSubsystemFacebook" property="bEnableAutomaticLogging" default="1"/>
		<setBoolFromProperty result="bEnableAdvertisingId" ini="Engine" section="OnlineSubsystemFacebook" property="bEnableAdvertisingId" default="1"/>
		<dumpvars/>
	</init>
	
	<registerBuildSettings>
	<insertValue value="OnlineSubsystemFacebook[bEnabled,ClientId,ClientToken,bEnableAutomaticLogging,bEnableAdvertisingId]"/>
	<insertNewline/>
	</registerBuildSettings>

	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
	</resourceCopies>
	
	<!-- Files to copy before the build so they can be used in the build -->
	<prebuildCopies>
		<if condition="bEnabled">
			<true>
				<log text="Copying Facebook buildtime files to staging"/>
				<copyDir src="$S(PluginDir)/ThirdParty/Android/Java"
					dst = "$S(BuildDir)/src/com/epicgames/unreal" /> 	
			</true>
		</if>
	</prebuildCopies>
	
	<!-- Facebook requires at least android-26 SDK platform for resources -->
	<minimumSDKAPI>
		<insertValue value="26"/>
		<insertNewline/>
	</minimumSDKAPI>
	
	<buildGradleAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
dependencies {
	implementation('com.facebook.android:facebook-android-sdk:18.0.2')
	implementation('com.facebook.android:facebook-login:18.0.2')
}
				</insert>
			</true>
		</if>
	</buildGradleAdditions>
	
	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<if condition="bEnabled">
			<true>
				<setElement result="FBAppId" value="meta-data" />
				<addAttribute tag="$FBAppId" name="android:name" value="com.facebook.sdk.ApplicationId" />
				<addAttribute tag="$FBAppId" name="android:value" value="fb$S(ClientId)" />
				<addElement tag="application" name="FBAppId" />

				<setElement result="FBClientToken" value="meta-data" />
				<addAttribute tag="$FBClientToken" name="android:name" value="com.facebook.sdk.ClientToken" />
				<addAttribute tag="$FBClientToken" name="android:value" value="$S(ClientToken)" />
				<addElement tag="application" name="FBClientToken" />

				<if condition="bEnableAutomaticLogging">
					<false>
						<setElement result="FBAnalyticsEnabled" value="meta-data" />
						<addAttribute tag="$FBAnalyticsEnabled" name="android:name" value="com.facebook.sdk.AutoLogAppEventsEnabled" />
						<addAttribute tag="$FBAnalyticsEnabled" name="android:value" value="false" />
						<addElement tag="application" name="FBAnalyticsEnabled" />
					</false>
				</if>

				<if condition="bEnableAdvertisingId">
					<false>
						<setElement result="FBAdIdCollectionEnabled" value="meta-data" />
						<addAttribute tag="$FBAdIdCollectionEnabled" name="android:name" value="com.facebook.sdk.AdvertiserIDCollectionEnabled" />
						<addAttribute tag="$FBAdIdCollectionEnabled" name="android:value" value="false" />
						<addElement tag="application" name="FBAdIdCollectionEnabled" />
					</false>
				</if>
				
				<addElements tag="application">
					<activity android:name="com.facebook.FacebookActivity"
        				android:configChanges="keyboard|keyboardHidden|screenLayout|screenSize|orientation"
        				android:label="@string/app_name" /> 
				</addElements>				

				<!-- Add permissions -->
				<addPermission android:name="android.permission.INTERNET" />
			</true>
		</if>
	</androidManifestUpdates>

	<!-- optional additions to proguard -->
	<proguardAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
-keep class epicgames.ue4.FacebookLogin.** {
	public *;
}  
				</insert>
			</true>
		</if>
	</proguardAdditions>

	<!-- optional files or directories to copy or delete from Intermediate/Android/APK before ndk-build 
	<prebuildCopies>
	</prebuildCopies> -->

	<!-- optional additions to the GameActivity imports in GameActivity.java -->
	<gameActivityImportAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
// Facebook imports
// import com.facebook.FacebookSdk;
				</insert>
			</true>
		</if>
	</gameActivityImportAdditions>

	<!-- optional additions to the GameActivity class in GameActivity.java -->
	<gameActivityClassAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook gameActivityClassAdditions
	private FacebookLogin facebookLogin;
	public FacebookLogin getFacebookLogin() { return facebookLogin; }

	public boolean AndroidThunkJava_Facebook_Login(String[] ScopeFields)
	{
		if (facebookLogin != null)
		{
			facebookLogin.login(ScopeFields);
		}
		return (facebookLogin != null);
	}

	public boolean AndroidThunkJava_Facebook_Logout()
	{
		if (facebookLogin != null)
		{
			facebookLogin.logout();
		}
		return (facebookLogin != null);
	}

	public String AndroidThunkJava_Facebook_GetAccessToken()
	{
		if (facebookLogin != null)
		{
			return facebookLogin.getAccessToken();
		}
		return "";
	}
	
	public boolean AndroidThunkJava_Facebook_RequestReadPermissions(String[] NewPermissions)
	{
		if (facebookLogin != null)
		{
			facebookLogin.requestReadPermissions(NewPermissions);
		}
		return (facebookLogin != null);
	}
	
	public boolean AndroidThunkJava_Facebook_RequestPublishPermissions(String[] NewPermissions)
	{
		if (facebookLogin != null)
		{
			facebookLogin.requestPublishPermissions(NewPermissions);
		}
		return (facebookLogin != null);
	}

	// End Facebook gameActivityClassAdditions
				</insert>
			</true>
		</if>
	</gameActivityClassAdditions>
	
	<!-- optional additions to GameActivity onCreate metadata reading in GameActivity.java -->
	<gameActivityReadMetadataAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
					// Facebook gameActivityReadMetadataAdditions
				</insert>
			</true>
		</if>
	</gameActivityReadMetadataAdditions>
	
	<!-- optional additions to GameActivity onCreate in GameActivity.java -->
	<gameActivityOnCreateAdditions>
		<if condition="bEnabled">
			<true>				
				<insertValue value="    boolean bEnableAppEvents = $B(bEnableAutomaticLogging);" />
				<insertNewline/>
				<insertValue value="    boolean bEnableAdId = $B(bEnableAdvertisingId);" />
				<insertNewline/>
				<insert>
					// Begin Facebook onCreate
					facebookLogin = new FacebookLogin(this, Log);
					if (!facebookLogin.init(BuildConfiguration, bEnableAppEvents, bEnableAdId))
					{
						facebookLogin = null;
						Log.error("Facebook SDK failed to initialize!");
					}
					else
					{
						Log.debug("Facebook SDK success!");
					}
					// End Facebook onCreate
				</insert>
			</true>
		</if>
	</gameActivityOnCreateAdditions>

	<!-- optional additions to GameActivity onDestroy in GameActivity.java-->
	<gameActivityOnDestroyAdditions>
		<if condition="bEnabled">
			<true>
				<insert>                    
	// Begin Facebook onDestroy
	if (facebookLogin != null)
	{
		facebookLogin.deactivate();
	}
	// End Facebook onDestroy
				</insert>
			</true>
		</if>
	</gameActivityOnDestroyAdditions>
	
	<!-- optional additions to GameActivity onStart in GameActivity.java -->
	<gameActivityOnStartAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook onStart
	// End Facebook onStart
				</insert>
			</true>
		</if>
	</gameActivityOnStartAdditions>

	<!-- optional additions to GameActivity onStop in GameActivity.java -->
	<gameActivityOnStopAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook onStop
	// End Facebook onStop
				</insert>
			</true>
		</if>
	</gameActivityOnStopAdditions>

	<!-- optional additions to GameActivity onPause in GameActivity.java -->
	<gameActivityOnPauseAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook onPause
	if (facebookLogin != null)
	{
		facebookLogin.deactivate();
	}
	// End Facebook onPause
				</insert>
			</true>
		</if>
	</gameActivityOnPauseAdditions>

	<!-- optional additions to GameActivity onResume in GameActivity.java -->
	<gameActivityOnResumeAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook onResume
	if (facebookLogin != null)
	{
		facebookLogin.activate();
	}
	// End Facebook onResume
				</insert>
			</true>
		</if>
	</gameActivityOnResumeAdditions>

	<!-- optional additions to GameActivity onActivityResult in GameActivity.java -->
	<gameActivityOnActivityResultAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// Begin Facebook onActivityResult
	if (facebookLogin != null)
	{
		facebookLogin.getCallbackManager().onActivityResult(requestCode, resultCode, data);
	}
	// End Facebook onActivityResult
				</insert>
			</true>
		</if>
	</gameActivityOnActivityResultAdditions>

	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	<soLoadLibrary>
	</soLoadLibrary>
</root>
