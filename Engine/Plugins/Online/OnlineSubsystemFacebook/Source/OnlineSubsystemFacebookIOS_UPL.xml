<?xml version="1.0" encoding="utf-8"?>
<!--OnlineSubsystemFacebook plugin additions-->
<root>
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Facebook SDK iOS init"/>
		
		<!-- get packaging for Facebook from ini and reset it if architecture not supported -->
		<setBoolFromProperty result="bEnabled" ini="Engine" section="OnlineSubsystemFacebook" property="bEnabled" default="false"/>
		<setStringFromProperty result="ClientId" ini="Engine" section="OnlineSubsystemFacebook" property="ClientId" default="0"/>
		<setStringFromProperty result="ClientToken" ini="Engine" section="OnlineSubsystemFacebook" property="ClientToken" default="0"/>
		<setStringFromProperty result="BundleDisplayName" ini="Engine" section="/Script/IOSRuntimeSettings.IOSRuntimeSettings" property="BundleDisplayName" default=""/>
		<setBoolFromProperty result="bEnableAutomaticLogging" ini="Engine" section="OnlineSubsystemFacebook" property="bEnableAutomaticLogging" default="1"/>
		<setBoolFromProperty result="bEnableAdvertisingId" ini="Engine" section="OnlineSubsystemFacebook" property="bEnableAdvertisingId" default="1"/>
		<dumpvars/>
	</init>

	<registerBuildSettings>
	<insertValue value="OnlineSubsystemFacebook[bEnabled,ClientId,ClientToken,bEnableAutomaticLogging,bEnableAdvertisingId]"/>
	<insertNewline/>
	</registerBuildSettings>

	<iosPListUpdates>
		<if condition="bEnabled">
			<true>
				<addElements tag="dict" once="true"> 
					<key>FacebookAppID</key>
				</addElements>
				<setElement result="NewEntry" value="string" text="$S(ClientId)"/>
				<addElement tag="dict" name="NewEntry" once="true"/> 

				<addElements tag="dict" once="true"> 
					<key>FacebookClientToken</key>
				</addElements>
				<setElement result="NewEntry" value="string" text="$S(ClientToken)"/>
				<addElement tag="dict" name="NewEntry" once="true"/> 

				<addElements tag="dict" once="true"> 
					<key>FacebookDisplayName</key>
				</addElements>
				<setElement result="NewEntry" value="string" text="$S(BundleDisplayName)"/>
				<addElement tag="dict" name="NewEntry" once="true"/> 

				<if condition="bEnableAutomaticLogging">
					<false>
						<addElements tag="dict" once="true">
							<key>FacebookAutoLogAppEventsEnabled</key>
							<false/>
						</addElements>
					</false>
				</if>

				<!-- CFBundleURLTypes block -->
				<!-- Insert our schemas into the first CFBundleURLSchemes array we find inside CFBundleURLTypes content-->
				<setBool result="bProcessingComplete" value="false"/>
				<loopElements tag="dict">
					<setBool result="bCFBundleURLTypesFound" value="false"/>

					<loopElements tag="$">
						<setStringFromTag result="TagName" tag="$"/>

						<!-- If we found CFBundleURLTypes we are on next iteration and be should be on its <array > value -->
						<if condition="bCFBundleURLTypesFound">
							<true>
								<setBoolIsEqual result="bIsArray" arg1="$S(TagName)" arg2="array"/>
								<if condition="bIsArray">
									<true>
										<!-- this element is an array of dictionaries -->
										<loopElements tag="$">
											<setBool result="bCFBundleURLSchemesFound" value="false"/>
											<!-- iterate on the <dict> entries looking for a <key> with the text CFBundleURLSchemes to insert our schemas-->
											<loopElements tag="$">
												<setStringFromTag result="TagName" tag="$"/>

												<!-- If we found CFBundleURLSchemes we are on next iteration and be should be on its <array > value, so we can add our schemas -->
												<if condition="bCFBundleURLSchemesFound">
													<true>
														<setBoolIsEqual result="bIsArray" arg1="$S(TagName)" arg2="array"/>
														<if condition="bIsArray">
															<true>
																<!-- Add schema to current array -->
																<setElement result="Schema" value="string" text="fb$S(ClientId)"/>
																<addElement tag="$" name="Schema"/>

																<setBool result="bProcessingComplete" value="true"/>
															</true>
														</if>
													</true>
												</if>

												<if condition="bProcessingComplete">
													<false>
														<!-- Check if we found a <key> with text CFBundleURLSchemes -->
														<setBoolIsEqual result="bIsKey" arg1="$S(TagName)" arg2="key"/>
														<if condition="bIsKey">
															<true>
																<setStringFromTagText result="TagValue" tag="$"/>
																<setBoolIsEqual result="bCFBundleURLSchemesFound" arg1="$S(TagValue)" arg2="CFBundleURLSchemes"/>
															</true>
														</if>
													</false>
												</if>
											</loopElements>
										</loopElements>
									</true>
								</if>
							</true>
						</if>

						<if condition="bProcessingComplete">
							<false>
								<!-- Check if we found a <key> with text CFBundleURLTypes -->
								<setBoolIsEqual result="bIsKey" arg1="$S(TagName)" arg2="key"/>
								<if condition="bIsKey">
									<true>
										<setStringFromTagText result="TagValue" tag="$"/>
										<setBoolIsEqual result="bCFBundleURLTypesFound" arg1="$S(TagValue)" arg2="CFBundleURLTypes"/>
									</true>
								</if>
							</false>
						</if>
					</loopElements>
				</loopElements>



				<!-- End of CFBundleURLTypes block -->

				<if condition="bEnableAdvertisingId">
					<false>
						<addElements tag="dict" once="true">
							<key>FacebookAdvertiserIDCollectionEnabled</key>
							<false/>
						</addElements>
					</false>
				</if>

				<addElements tag="dict" once="true">
					<key>LSApplicationQueriesSchemes</key>
					<array>
						<string>fbapi</string>
						<string>fb-messenger-api</string>
						<string>fbauth2</string>
						<string>fbshareextension</string>
					</array>
				</addElements>
			</true>
		</if>
	</iosPListUpdates>
	
</root>
