// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIInterfaceFacebook.h"
#include "OnlineSubsystemFacebook.h"
#include "OnlineIdentityFacebook.h"
#include "OnlineError.h"

FOnlineExternalUIFacebook::FOnlineExternalUIFacebook(FOnlineSubsystemFacebook* InSubsystem) :
	FOnlineExternalUIFacebookCommon(InSubsystem)
{
}

FOnlineExternalUIFacebook::~FOnlineExternalUIFacebook()
{
}

bool FOnlineExternalUIFacebook::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	IOnlineIdentityPtr OnlineIdentity = FacebookSubsystem->GetIdentityInterface();

	if (FUniqueNetIdPtr UniqueNetId = OnlineIdentity->GetUniquePlayerId(ControllerIndex); UniqueNetId && UniqueNetId->IsValid())
	{
		Delegate.ExecuteIfBound(UniqueNetId, ControllerIndex, FOnlineError::Success());
		return true;
	}

	TSharedPtr<FDelegateHandle> DelegateHandle = MakeShared<FDelegateHandle>();
	*DelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(ControllerIndex, FOnLoginCompleteDelegate::CreateLambda([this, DelegateHandle, Delegate](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString)
		{
			FOnlineError Error(bWasSuccessful);
			Error.SetFromErrorCode(ErrorString);

			Delegate.ExecuteIfBound(UserId.IsValid()? UserId.AsShared() : FUniqueNetIdPtr(), LocalUserNum, Error);

			IOnlineIdentityPtr OnlineIdentity = FacebookSubsystem->GetIdentityInterface();
			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *DelegateHandle);
		}));
	OnlineIdentity->Login(ControllerIndex, FOnlineAccountCredentials());
	return true;
}