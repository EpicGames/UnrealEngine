// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemFacebook.h"
#include "OnlineExternalUIFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/** 
 * Implementation for the Facebook external UIs
 */
class FOnlineExternalUIFacebook : public FOnlineExternalUIFacebookCommon
{
private:

PACKAGE_SCOPE:

	/** 
	 * Constructor
	 * @param InSubsystem The owner of this external UI interface.
	 */
	 FOnlineExternalUIFacebook(FOnlineSubsystemFacebook* InSubsystem);

public:

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUIFacebook();

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
};

typedef TSharedPtr<FOnlineExternalUIFacebook, ESPMode::ThreadSafe> FOnlineExternalUIFacebookPtr;

