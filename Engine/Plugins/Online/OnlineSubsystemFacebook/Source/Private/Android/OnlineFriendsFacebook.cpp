// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFriendsFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"

#if WITH_FACEBOOK

FOnlineFriendsFacebook::FOnlineFriendsFacebook(FOnlineSubsystemFacebook* InSubsystem) 
	: FOnlineFriendsFacebookCommon(InSubsystem)
{
}

FOnlineFriendsFacebook::~FOnlineFriendsFacebook()
{
}

#endif // WITH_FACEBOOK

