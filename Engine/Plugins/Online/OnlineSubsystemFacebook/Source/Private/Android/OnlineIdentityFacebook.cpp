// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityFacebook.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemFacebookPrivate.h"

#if WITH_FACEBOOK

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "Async/TaskGraphInterfaces.h"
#include "OnlineJniFacebookLogin.h"

FOnlineIdentityFacebook::FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineIdentityFacebookCommon(InSubsystem)
{
	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("ScopeFields"), ScopeFields, GEngineIni);
	// always required login access fields
	ScopeFields.AddUnique(TEXT(PERM_PUBLIC_PROFILE));

	FOnFacebookLoginCompleteDelegate LoginDelegate = FOnFacebookLoginCompleteDelegate::CreateRaw(this, &FOnlineIdentityFacebook::OnLoginComplete);
	OnFBLoginCompleteHandle = AddOnFacebookLoginCompleteDelegate_Handle(LoginDelegate);

	FOnFacebookLogoutCompleteDelegate LogoutDelegate = FOnFacebookLogoutCompleteDelegate::CreateRaw(this, &FOnlineIdentityFacebook::OnLogoutComplete);
	OnFBLogoutCompleteHandle = AddOnFacebookLogoutCompleteDelegate_Handle(LogoutDelegate);
}

bool FOnlineIdentityFacebook::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	bool bTriggeredLogin = false;
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
		if (LoginStatus == ELoginStatus::NotLoggedIn)
		{
			LoginCompletionDelegate = FOnInternalLoginComplete::CreateLambda(
				[this, LocalUserNum](EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
			{
				UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnInternalLoginComplete %s %s"), ToString(InResponseCode), *InAccessToken);
				if (InResponseCode == EFacebookLoginResponse::RESPONSE_OK &&
					!InAccessToken.IsEmpty())
				{
					TSharedPtr<FOnlineSharingFacebook> Sharing = StaticCastSharedPtr<FOnlineSharingFacebook>(FacebookSubsystem->GetSharingInterface());
					Sharing->SetCurrentPermissions(GrantedPermissions, DeclinedPermissions);
					FOnProfileRequestComplete CompletionDelegate = FOnProfileRequestComplete::CreateLambda([this](int32 LocalUserNumFromRequest, bool bWasProfileRequestSuccessful, const FString& ErrorStr)
						{
							OnLoginAttemptComplete(LocalUserNumFromRequest, ErrorStr);
						});

					ProfileRequest(LocalUserNum, InAccessToken, ProfileFields, CompletionDelegate);
				}
				else
				{
					FString ErrorStr;
					if (InResponseCode == EFacebookLoginResponse::RESPONSE_CANCELED)
					{
						ErrorStr = LOGIN_CANCELLED;
					}
					else
					{
						ErrorStr = FString::Printf(TEXT("Login failure %s"), ToString(InResponseCode));
					}
					OnLoginAttemptComplete(LocalUserNum, ErrorStr);
				}
			});

			extern bool AndroidThunkCpp_Facebook_Login(const TArray<FString>&);
			bTriggeredLogin = AndroidThunkCpp_Facebook_Login(ScopeFields);
			if (!bTriggeredLogin)
			{
				// Facebook SDK was not properly initialized or JEnv is wrong
				OnLoginFailed();
			}
		}
		else
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *GetUniquePlayerId(LocalUserNum), TEXT("Already logged in"));
		}
	}
	else
	{
		FString ErrorStr = FString::Printf(TEXT("Operation already in progress"));
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
	}

	return bTriggeredLogin;
}

void FOnlineIdentityFacebook::OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr)
{
	const FString ErrorStrCopy(ErrorStr);

	if (GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		extern FString AndroidThunkCpp_Facebook_GetAccessToken();
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Facebook login was successful %s"), *AndroidThunkCpp_Facebook_GetAccessToken());
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());
		
		FacebookSubsystem->ExecuteNextTick([this, UserId, LocalUserNum, ErrorStrCopy]()
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, ErrorStrCopy);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
		});
	}
	else
	{
		LogoutCompletionDelegate = FOnInternalLogoutComplete::CreateLambda(
			[this, LocalUserNum, ErrorStrCopy](EFacebookLoginResponse InResponseCode)
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Facebook login failed: %s"), *ErrorStrCopy);

			FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
			if (UserId.IsValid())
			{
				// remove cached user account
				UserAccounts.Remove(UserId->ToString());
			}
			else
			{
				UserId = FUniqueNetIdFacebook::EmptyId();
			}
			// remove cached user id
			UserIds.Remove(LocalUserNum);

			TriggerOnLoginCompleteDelegates(LocalUserNum, false, *UserId, ErrorStrCopy);
		});

		// Clean up anything left behind from cached access tokens
		extern bool AndroidThunkCpp_Facebook_Logout();
		if (!AndroidThunkCpp_Facebook_Logout())
		{
			// Facebook SDK not properly initialized or JEnv is wrong
			OnLogoutComplete(EFacebookLoginResponse::RESPONSE_ERROR);
		}
	}
}

bool FOnlineIdentityFacebook::Logout(int32 LocalUserNum)
{
	bool bTriggeredLogout = false;
	bool bPendingOp = LoginCompletionDelegate.IsBound() || LogoutCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		ELoginStatus::Type LoginStatus = GetLoginStatus(LocalUserNum);
		if (LoginStatus == ELoginStatus::LoggedIn)
		{
			LogoutCompletionDelegate = FOnInternalLogoutComplete::CreateLambda(
				[this, LocalUserNum](EFacebookLoginResponse InResponseCode)
			{
				UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("FOnInternalLogoutComplete %s"), ToString(InResponseCode));
				FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
				if (UserId.IsValid())
				{
					// remove cached user account
					UserAccounts.Remove(UserId->ToString());
				}
				else
				{
					UserId = FUniqueNetIdFacebook::EmptyId();
				}
				// remove cached user id
				UserIds.Remove(LocalUserNum);

				FacebookSubsystem->ExecuteNextTick([this, UserId, LocalUserNum]()
				{
					TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
					TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
				});
			});

			extern bool AndroidThunkCpp_Facebook_Logout();
			bTriggeredLogout = AndroidThunkCpp_Facebook_Logout();
			if (!bTriggeredLogout)
			{
				// Facebook SDK not properly initialized or JEnv is wrong
				OnLogoutComplete(EFacebookLoginResponse::RESPONSE_ERROR);
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		}
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Operation already in progress"));
	}

	if (!bTriggeredLogout)
	{
		FacebookSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}

	return bTriggeredLogout;
}

void FOnlineIdentityFacebook::OnLoginComplete(EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnLoginComplete %s %s"), ToString(InResponseCode), *InAccessToken);
	LoginCompletionDelegate.ExecuteIfBound(InResponseCode, InAccessToken, GrantedPermissions, DeclinedPermissions);
	LoginCompletionDelegate.Unbind();
}

void FOnlineIdentityFacebook::OnLoginFailed()
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnLoginFailed"));
	LoginCompletionDelegate.ExecuteIfBound(EFacebookLoginResponse::RESPONSE_ERROR, "", {}, {});
	LoginCompletionDelegate.Unbind();
}

void FOnlineIdentityFacebook::OnLogoutComplete(EFacebookLoginResponse InResponseCode)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("OnLogoutComplete %s"), ToString(InResponseCode));
	LogoutCompletionDelegate.ExecuteIfBound(InResponseCode);
	LogoutCompletionDelegate.Unbind();
}

#define CHECK_JNI_METHOD(Id) checkf(Id != nullptr, TEXT("Failed to find " #Id));

FString AndroidThunkCpp_Facebook_GetAccessToken()
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("AndroidThunkCpp_Facebook_GetAccessToken"));

	FString AccessToken;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		const bool bIsOptional = false;
		static jmethodID FacebookGetAccessTokenMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Facebook_GetAccessToken", "()Ljava/lang/String;", bIsOptional);
		CHECK_JNI_METHOD(FacebookGetAccessTokenMethod);
		
		AccessToken = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FacebookGetAccessTokenMethod));
	}
	
	return AccessToken;
}

bool AndroidThunkCpp_Facebook_Login(const TArray<FString>& InScopeFields)
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("AndroidThunkCpp_Facebook_Login"));
	bool bSuccess = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		const bool bIsOptional = false;
		static jmethodID FacebookLoginMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Facebook_Login", "([Ljava/lang/String;)Z", bIsOptional);
		CHECK_JNI_METHOD(FacebookLoginMethod);

		// Convert scope array into java fields
		auto ScopeIDArray = NewScopedJavaObject(Env, (jobjectArray)Env->NewObjectArray(InScopeFields.Num(), FJavaWrapper::JavaStringClass, nullptr));
		for (uint32 Param = 0; Param < InScopeFields.Num(); Param++)
		{
			auto StringValue = FJavaHelper::ToJavaString(Env, InScopeFields[Param]);
			Env->SetObjectArrayElement(*ScopeIDArray, Param, *StringValue);
		}

		bSuccess = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FacebookLoginMethod, *ScopeIDArray);		
	}

	return bSuccess;
}

void JNICALL UE::Jni::FFacebookLogin::nativeLoginComplete(JNIEnv* jenv, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions)
{
	EFacebookLoginResponse LoginResponse = (EFacebookLoginResponse)responseCode;

	auto AccessToken = FJavaHelper::FStringFromParam(jenv, accessToken);
	TArray<FString> GrantedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, grantedPermissions);
	TArray<FString> DeclinedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, declinedPermissions);

	UE_LOG_ONLINE_IDENTITY(VeryVerbose, TEXT("nativeLoginComplete Response: %d Token: %s"), (int)LoginResponse, *AccessToken);

	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.ProcessFacebookLogin"), STAT_FSimpleDelegateGraphTask_ProcessFacebookLogin, STATGROUP_TaskGraphTasks);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]()
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Facebook login completed %s\n"), ToString(LoginResponse));
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::Get(FACEBOOK_SUBSYSTEM))
			{
				FOnlineIdentityFacebookPtr IdentityFBInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(OnlineSub->GetIdentityInterface());
				if (IdentityFBInt.IsValid())
				{
					IdentityFBInt->TriggerOnFacebookLoginCompleteDelegates(LoginResponse, AccessToken, GrantedPermissions, DeclinedPermissions);
				}
			}
		}),
	GET_STATID(STAT_FSimpleDelegateGraphTask_ProcessFacebookLogin),
	nullptr,
	ENamedThreads::GameThread
	);
}

bool AndroidThunkCpp_Facebook_Logout()
{
	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("AndroidThunkCpp_Facebook_Logout"));
	bool bSuccess = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		const bool bIsOptional = false;
		static jmethodID FacebookLogoutMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Facebook_Logout", "()Z", bIsOptional);
		CHECK_JNI_METHOD(FacebookLogoutMethod);
		bSuccess = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FacebookLogoutMethod);
	}
	
	return bSuccess;
}

void JNICALL UE::Jni::FFacebookLogin::nativeLogoutComplete(JNIEnv* jenv, jobject thiz, jsize responseCode)
{
	EFacebookLoginResponse LogoutResponse = (EFacebookLoginResponse)responseCode;
	UE_LOG_ONLINE_IDENTITY(VeryVerbose, TEXT("nativeLogoutComplete %s"), ToString(LogoutResponse));

	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.ProcessFacebookLogout"), STAT_FSimpleDelegateGraphTask_ProcessFacebookLogout, STATGROUP_TaskGraphTasks);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]()
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Facebook logout completed %s\n"), ToString(LogoutResponse));
			if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::Get(FACEBOOK_SUBSYSTEM))
			{
				FOnlineIdentityFacebookPtr IdentityFBInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(OnlineSub->GetIdentityInterface());
				if (IdentityFBInt.IsValid())
				{
					IdentityFBInt->TriggerOnFacebookLogoutCompleteDelegates(LogoutResponse);
				}
			}
		}),
	GET_STATID(STAT_FSimpleDelegateGraphTask_ProcessFacebookLogout),
	nullptr,
	ENamedThreads::GameThread
	);
}

#endif // WITH_FACEBOOK



