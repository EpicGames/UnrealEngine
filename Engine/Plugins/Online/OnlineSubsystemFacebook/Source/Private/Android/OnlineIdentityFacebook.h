// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "OnlineIdentityFacebookCommon.h"
#include "OnlineSharingFacebook.h"
#include "OnlineAccountFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/** Tied to FacebookLogin.java */
enum class EFacebookLoginResponse : uint8
{
	/** Facebook SDK ok response */
	RESPONSE_OK = 0,
	/** Facebook SDK user cancellation */
	RESPONSE_CANCELED = 1,
	/** Facebook SDK error */
	RESPONSE_ERROR = 2,
};

inline const TCHAR* ToString(EFacebookLoginResponse Response)
{
	switch (Response)
	{
		case EFacebookLoginResponse::RESPONSE_OK:
			return TEXT("RESPONSE_OK");
		case EFacebookLoginResponse::RESPONSE_CANCELED:
			return TEXT("RESPONSE_CANCELED");
		case EFacebookLoginResponse::RESPONSE_ERROR:
			return TEXT("RESPONSE_ERROR");
	}

	return TEXT("");
}

/**
 * Delegate fired when the Facebook Android SDK has completed a login request
 *
 * @param InResponseCode response from the Facebook SDK
 * @param InAccessToken access token if the response was RESPONSE_OK
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnFacebookLoginComplete, EFacebookLoginResponse /*InResponseCode*/, const FString& /*InAccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);
typedef FOnFacebookLoginComplete::FDelegate FOnFacebookLoginCompleteDelegate;

/**
 * Delegate fired when the Facebook Android SDK has completed a logout request
 *
 * @param InResponseCode response from the Facebook SDK
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFacebookLogoutComplete, EFacebookLoginResponse /*InResponseCode*/);
typedef FOnFacebookLogoutComplete::FDelegate FOnFacebookLogoutCompleteDelegate;


/** Android implementation of a Facebook user account */
using FUserOnlineAccountFacebook  = FUserOnlineAccountFacebookCommon;

/**
 * Facebook service implementation of the online identity interface
 */
class FOnlineIdentityFacebook :
	public FOnlineIdentityFacebookCommon
{

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;

	// FOnlineIdentityFacebook

	FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityFacebook()
	{
	}

PACKAGE_SCOPE:

	/**
	 * Delegate fired internally when the Java Facebook SDK has completed, notifying any OSS listeners
	 * Not meant for external use
	 *
	 * @param InResponseCode response from the FacebookSDK
	 * @param InAccessToken access token from the FacebookSDK if login was successful
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnFacebookLoginComplete, EFacebookLoginResponse /*InResponseCode*/, const FString& /*InAccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);

	/**
	 * Delegate fired internally when the Java Facebook SDK has completed, notifying any OSS listeners
	 * Not meant for external use
	 *
	 * @param InResponseCode response from the FacebookSDK
	 */
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnFacebookLogoutComplete, EFacebookLoginResponse /*InResponseCode*/);


private:

	/** Last function called for any single login attempt */
	void OnLoginAttemptComplete(int32 LocalUserNum, const FString& ErrorStr);

	/** Generic handler for the Java SDK login callback */
	void OnLoginComplete(EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions);

	void OnLoginFailed();
	/** Generic handler for the Java SDK logout callback */
	void OnLogoutComplete(EFacebookLoginResponse InResponseCode);

	/** Delegate holder for all internal related login callbacks */
	DECLARE_DELEGATE_FourParams(FOnInternalLoginComplete, EFacebookLoginResponse /*InLoginResponse*/, const FString& /*AccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);
	FOnInternalLoginComplete LoginCompletionDelegate;
	/** Delegate holder for all internal related logout callbacks */
	DECLARE_DELEGATE_OneParam(FOnInternalLogoutComplete, EFacebookLoginResponse /*InLoginResponse*/);
	FOnInternalLogoutComplete LogoutCompletionDelegate;

	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;

	FDelegateHandle OnFBLoginCompleteHandle;
	FDelegateHandle OnFBLogoutCompleteHandle;
};

typedef TSharedPtr<FOnlineIdentityFacebook, ESPMode::ThreadSafe> FOnlineIdentityFacebookPtr;
