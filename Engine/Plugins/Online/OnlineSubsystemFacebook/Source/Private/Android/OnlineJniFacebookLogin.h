// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "CoreMinimal.h"
#if USE_ANDROID_JNI && WITH_FACEBOOK
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FFacebookLogin: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/FacebookLogin";

		static void JNICALL nativeLoginComplete(JNIEnv* env, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions);
		static void JNICALL nativeRequestReadPermissionsComplete(JNIEnv* env, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions);
		static void JNICALL nativeRequestPublishPermissionsComplete(JNIEnv* env, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions);
		static void JNICALL nativeLogoutComplete(JNIEnv* env, jobject thiz, jsize responseCode);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeLoginComplete),
			UE_JNI_NATIVE_METHOD(nativeRequestReadPermissionsComplete),
			UE_JNI_NATIVE_METHOD(nativeRequestPublishPermissionsComplete),
			UE_JNI_NATIVE_METHOD(nativeLogoutComplete)
		};
	};
	
	template struct TInitialize<FFacebookLogin>;
}
#endif
