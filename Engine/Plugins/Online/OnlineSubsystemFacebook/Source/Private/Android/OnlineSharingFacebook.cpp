// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineSharingFacebook.h"
#include "OnlineJniFacebookLogin.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebook.h"

#if WITH_FACEBOOK

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Async/TaskGraphInterfaces.h"

FOnlineSharingFacebook::FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineSharingFacebookCommon(InSubsystem)
{
	FOnFacebookRequestPermissionsOpCompleteDelegate RequestPermissionsOpDelegate = FOnFacebookRequestPermissionsOpCompleteDelegate::CreateRaw(this, &FOnlineSharingFacebook::OnPermissionsOpComplete);
	OnFBRequestPermissionsOpCompleteHandle = AddOnFacebookRequestPermissionsOpCompleteDelegate_Handle(RequestPermissionsOpDelegate);
}

FOnlineSharingFacebook::~FOnlineSharingFacebook()
{
}

bool FOnlineSharingFacebook::RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions)
{
	ensure((NewPermissions & ~EOnlineSharingCategory::ReadPermissionMask) == EOnlineSharingCategory::None);

	bool bTriggeredRequest = false;
	bool bPendingOp = PermissionsOpCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
		if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
		{
			// Here we iterate over each category, adding each individual permission linked with it in the ::SetupPermissionMaps
			TArray<FSharingPermission> PermissionsNeeded;
			const bool bHasPermission = CurrentPermissions.HasPermission(NewPermissions, PermissionsNeeded);
			if (!bHasPermission)
			{
				PermissionsOpCompletionDelegate = FOnPermissionsOpComplete::CreateLambda([this, LocalUserNum](EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
					{
						UE_LOG_ONLINE(Display, TEXT("RequestNewReadPermissions : %s"), ToString(InResponseCode));
						const bool bIsOk = (InResponseCode == EFacebookLoginResponse::RESPONSE_OK);
						if (bIsOk)
						{
							CurrentPermissions.RefreshPermissions(GrantedPermissions, DeclinedPermissions);
						}
						TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, bIsOk);
					});

				extern bool AndroidThunkCpp_Facebook_RequestReadPermissions(const TArray<FSharingPermission>&);
				bTriggeredRequest = AndroidThunkCpp_Facebook_RequestReadPermissions(PermissionsNeeded);
				if (!bTriggeredRequest)
				{
					// Facebook SDK was not properly initialized or JEnv is wrong
					OnPermissionsOpFailed();
				}
			}
			else
			{
				// All permissions were already granted, no need to reauthorize
				TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, true);
			}
		}
		else
		{
			// If we weren't logged into Facebook we cannot do this action
			TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
		}
	}
	else
	{
		UE_LOG_ONLINE_SHARING(Verbose, TEXT("Operation already in progress"));
		TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
	}

	return bTriggeredRequest;
}

bool FOnlineSharingFacebook::RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy)
{
	ensure((NewPermissions & ~EOnlineSharingCategory::PublishPermissionMask) == EOnlineSharingCategory::None);

	bool bTriggeredRequest = false;
	bool bPendingOp = PermissionsOpCompletionDelegate.IsBound();
	if (!bPendingOp)
	{
		IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
		if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
		{
			// Here we iterate over each category, adding each individual permission linked with it in the ::SetupPermissionMaps
			TArray<FSharingPermission> PermissionsNeeded;
			const bool bHasPermission = CurrentPermissions.HasPermission(NewPermissions, PermissionsNeeded);
			if (!bHasPermission)
			{
				PermissionsOpCompletionDelegate = FOnPermissionsOpComplete::CreateLambda([this, LocalUserNum](EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
					{
						UE_LOG_ONLINE(Display, TEXT("RequestNewPublishPermissions : %s"), ToString(InResponseCode));
						const bool bIsOk = (InResponseCode == EFacebookLoginResponse::RESPONSE_OK);
						if (bIsOk)
						{
							CurrentPermissions.RefreshPermissions(GrantedPermissions, DeclinedPermissions);
						}
						TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, bIsOk);
					});

				extern bool AndroidThunkCpp_Facebook_RequestPublishPermissions(const TArray<FSharingPermission>&);
				bTriggeredRequest = AndroidThunkCpp_Facebook_RequestPublishPermissions(PermissionsNeeded);
				if (!bTriggeredRequest)
				{
					// Facebook SDK was not properly initialized or JEnv is wrong
					OnPermissionsOpFailed();
				}
			}
			else
			{
				// All permissions were already granted, no need to reauthorize
				TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, true);
			}
		}
		else
		{
			// If we weren't logged into Facebook we cannot do this action
			TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
		}
	}
	else
	{
		UE_LOG_ONLINE_SHARING(Verbose, TEXT("Operation already in progress"));
		TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
	}

	return bTriggeredRequest;
}

void FOnlineSharingFacebook::OnPermissionsOpFailed()
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("OnPermissionsOpFailed"));
	PermissionsOpCompletionDelegate.ExecuteIfBound(EFacebookLoginResponse::RESPONSE_ERROR, TEXT(""), {}, {});
	PermissionsOpCompletionDelegate.Unbind();
}

void FOnlineSharingFacebook::OnPermissionsOpComplete(EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("OnPermissionsOpComplete %s %s"), ToString(InResponseCode), *InAccessToken);
	PermissionsOpCompletionDelegate.ExecuteIfBound(InResponseCode, InAccessToken, GrantedPermissions, DeclinedPermissions);
	PermissionsOpCompletionDelegate.Unbind();
}

#define CHECK_JNI_METHOD(Id) checkf(Id != nullptr, TEXT("Failed to find " #Id));

bool AndroidThunkCpp_Facebook_RequestReadPermissions(const TArray<FSharingPermission>& InNewPermissions)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("AndroidThunkCpp_Facebook_RequestReadPermissions"));
	bool bSuccess = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		const bool bIsOptional = false;
		static jmethodID FacebookRequestReadMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Facebook_RequestReadPermissions", "([Ljava/lang/String;)Z", bIsOptional);
		CHECK_JNI_METHOD(FacebookRequestReadMethod);

		// Convert scope array into java fields
		auto PermsIDArray = NewScopedJavaObject(Env, (jobjectArray)Env->NewObjectArray(InNewPermissions.Num(), FJavaWrapper::JavaStringClass, nullptr));
		for (uint32 Param = 0; Param < InNewPermissions.Num(); Param++)
		{
			auto StringValue = FJavaHelper::ToJavaString(Env, InNewPermissions[Param].Name);
			Env->SetObjectArrayElement(*PermsIDArray, Param, *StringValue);
		}

		bSuccess = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FacebookRequestReadMethod, *PermsIDArray);
	}

	return bSuccess;
}

void JNICALL UE::Jni::FFacebookLogin::nativeRequestReadPermissionsComplete(JNIEnv* jenv, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions)
{
	EFacebookLoginResponse LoginResponse = (EFacebookLoginResponse)responseCode;

	auto AccessToken = FJavaHelper::FStringFromParam(jenv, accessToken);
	TArray<FString> GrantedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, grantedPermissions);
	TArray<FString> DeclinedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, declinedPermissions);

	UE_LOG_ONLINE_SHARING(VeryVerbose, TEXT("nativeRequestReadPermissionsComplete Response: %d Token: %s"), (int)LoginResponse, *AccessToken);

	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.ProcessFacebookReadPermissions"), STAT_FSimpleDelegateGraphTask_ProcessFacebookReadPermissions, STATGROUP_TaskGraphTasks);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]()
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Facebook request read permissions completed %s\n"), ToString(LoginResponse));
				if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::Get(FACEBOOK_SUBSYSTEM))
				{
					FOnlineSharingFacebookPtr SharingFBInt = StaticCastSharedPtr<FOnlineSharingFacebook>(OnlineSub->GetSharingInterface());
					if (SharingFBInt.IsValid())
					{
						SharingFBInt->TriggerOnFacebookRequestPermissionsOpCompleteDelegates(LoginResponse, AccessToken, GrantedPermissions, DeclinedPermissions);
					}
				}
			}),
		GET_STATID(STAT_FSimpleDelegateGraphTask_ProcessFacebookReadPermissions),
				nullptr,
				ENamedThreads::GameThread
				);
}

bool AndroidThunkCpp_Facebook_RequestPublishPermissions(const TArray<FSharingPermission>& InNewPermissions)
{
	UE_LOG_ONLINE_SHARING(Verbose, TEXT("AndroidThunkCpp_Facebook_RequestPublishPermissions"));
	bool bSuccess = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		const bool bIsOptional = false;
		static jmethodID FacebookRequestPublishMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Facebook_RequestPublishPermissions", "([Ljava/lang/String;)Z", bIsOptional);
		CHECK_JNI_METHOD(FacebookRequestPublishMethod);

		// Convert scope array into java fields
		auto PermsIDArray = NewScopedJavaObject(Env, (jobjectArray)Env->NewObjectArray(InNewPermissions.Num(), FJavaWrapper::JavaStringClass, nullptr));
		for (uint32 Param = 0; Param < InNewPermissions.Num(); Param++)
		{
			auto StringValue = FJavaHelper::ToJavaString(Env, InNewPermissions[Param].Name);
			Env->SetObjectArrayElement(*PermsIDArray, Param, *StringValue);
		}

		bSuccess = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FacebookRequestPublishMethod, *PermsIDArray);
	}

	return bSuccess;
}

void JNICALL UE::Jni::FFacebookLogin::nativeRequestPublishPermissionsComplete(JNIEnv* jenv, jobject thiz, jsize responseCode, jstring accessToken, Java::Lang::TArray<jstring>* grantedPermissions, Java::Lang::TArray<jstring>* declinedPermissions)
{
	EFacebookLoginResponse LoginResponse = (EFacebookLoginResponse)responseCode;

	auto AccessToken = FJavaHelper::FStringFromParam(jenv, accessToken);
	TArray<FString> GrantedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, grantedPermissions);
	TArray<FString> DeclinedPermissions = FJavaHelper::ObjectArrayToFStringTArray(jenv, declinedPermissions);

	UE_LOG_ONLINE_SHARING(VeryVerbose, TEXT("nativeRequestPublishPermissionsComplete Response: %d Token: %s"), (int)LoginResponse, *AccessToken);

	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.ProcessFacebookPublishPermissions"), STAT_FSimpleDelegateGraphTask_ProcessFacebookPublishPermissions, STATGROUP_TaskGraphTasks);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]()
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Facebook request publish permissions completed %s\n"), ToString(LoginResponse));
				if (IOnlineSubsystem* const OnlineSub = IOnlineSubsystem::Get(FACEBOOK_SUBSYSTEM))
				{
					FOnlineSharingFacebookPtr SharingFBInt = StaticCastSharedPtr<FOnlineSharingFacebook>(OnlineSub->GetSharingInterface());
					if (SharingFBInt.IsValid())
					{
						SharingFBInt->TriggerOnFacebookRequestPermissionsOpCompleteDelegates(LoginResponse, AccessToken, GrantedPermissions, DeclinedPermissions);
					}
				}
			}),
		GET_STATID(STAT_FSimpleDelegateGraphTask_ProcessFacebookPublishPermissions),
				nullptr,
				ENamedThreads::GameThread
				);
}

#endif // WITH_FACEBOOK
