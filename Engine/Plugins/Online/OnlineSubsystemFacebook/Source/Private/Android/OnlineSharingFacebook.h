// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
// Module includes
#include "OnlineSharingFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;
enum class EFacebookLoginResponse : uint8;

/**
 * Delegate fired when the Facebook Android SDK has completed a permissions update request
 *
 * @param InResponseCode response from the Facebook SDK
 * @param InAccessToken access token if the response was RESPONSE_OK
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnFacebookRequestPermissionsOpComplete, EFacebookLoginResponse /*InResponseCode*/, const FString& /*InAccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);
typedef FOnFacebookRequestPermissionsOpComplete::FDelegate FOnFacebookRequestPermissionsOpCompleteDelegate;

/**
 * Facebook implementation of the Online Sharing Interface
 */
class FOnlineSharingFacebook : public FOnlineSharingFacebookCommon
{

public:

	//~ Begin IOnlineSharing Interface
	virtual bool RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions) override;
	virtual bool RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy) override;
	//~ End IOnlineSharing Interface

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineSharingFacebook();

PACKAGE_SCOPE:

	/**
	 * Delegate fired internally when the Java Facebook SDK has completed, notifying any OSS listeners
	 * Not meant for external use
	 *
	 * @param InResponseCode response from the FacebookSDK
	 * @param InAccessToken access token from the FacebookSDK if login was successful
	 * @param GrantedPermissions list of Facebook permission names granted when using the token
	 * @param DeclinedPermissions list of Facebook permission names declined when using the token
	 */
	DEFINE_ONLINE_DELEGATE_FOUR_PARAM(OnFacebookRequestPermissionsOpComplete, EFacebookLoginResponse /*InResponseCode*/, const FString& /*InAccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);

private:

	/** Generic handler for both read/publish permissions requests */
	void OnPermissionsOpFailed();
	void OnPermissionsOpComplete(EFacebookLoginResponse InResponseCode, const FString& InAccessToken, const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions);

	/** Delegate holder for all internal related permissions callbacks */
	DECLARE_DELEGATE_FourParams(FOnPermissionsOpComplete, EFacebookLoginResponse /*InLoginResponse*/, const FString& /*AccessToken*/, const TArray<FString>& /*GrantedPermissions*/, const TArray<FString>& /*DeclinedPermissions*/);
	FOnPermissionsOpComplete PermissionsOpCompletionDelegate;

	FDelegateHandle OnFBRequestPermissionsOpCompleteHandle;
};

typedef TSharedPtr<FOnlineSharingFacebook, ESPMode::ThreadSafe> FOnlineSharingFacebookPtr;