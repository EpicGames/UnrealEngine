// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineExternalUIInterfaceFacebook.h"
#include "OnlineIdentityFacebook.h"
#include "OnlineFriendsFacebook.h"
#include "OnlineSharingFacebook.h"
#include "OnlineUserFacebook.h"

FOnlineSubsystemFacebook::FOnlineSubsystemFacebook(FName InInstanceName)
	: FOnlineSubsystemFacebookCommon(InInstanceName)
{
}

FOnlineSubsystemFacebook::~FOnlineSubsystemFacebook()
{
}

bool FOnlineSubsystemFacebook::Init()
{
#if WITH_FACEBOOK
	FacebookIdentity = MakeShared<FOnlineIdentityFacebook>(this);
	FacebookFriends = MakeShared<FOnlineFriendsFacebook>(this);
	FacebookSharing = MakeShared<FOnlineSharingFacebook>(this);
    FacebookUser = MakeShared<FOnlineUserFacebook>(this);
	FacebookExternalUI = MakeShared<FOnlineExternalUIFacebook>(this);
	return true;
#else
	return false;
#endif
}

bool FOnlineSubsystemFacebook::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemFacebook::Shutdown()"));
	return FOnlineSubsystemFacebookCommon::Shutdown();
}

bool FOnlineSubsystemFacebook::IsEnabled() const
{
	// Overridden due to different platform implementations of IsEnabled
	return FOnlineSubsystemFacebookCommon::IsEnabled();
}
