// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineUserFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"

#if WITH_FACEBOOK

FOnlineUserFacebook::FOnlineUserFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineUserFacebookCommon(InSubsystem)
{
}

FOnlineUserFacebook::~FOnlineUserFacebook()
{
}

#endif // WITH_FACEBOOK
