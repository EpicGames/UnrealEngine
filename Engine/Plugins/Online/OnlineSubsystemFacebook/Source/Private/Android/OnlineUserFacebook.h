// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
// Module includes
#include "OnlineUserFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/**
 * Facebook implementation of the Online User Interface
 */
class FOnlineUserFacebook : public FOnlineUserFacebookCommon
{

public:
	
	// FOnlineUserFacebook

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineUserFacebook(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineUserFacebook();

};

typedef TSharedPtr<FOnlineUserFacebook, ESPMode::ThreadSafe> FOnlineUserFacebookPtr;