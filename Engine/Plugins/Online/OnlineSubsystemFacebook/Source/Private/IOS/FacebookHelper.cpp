// Copyright Epic Games, Inc. All Rights Reserved.

#include "FacebookHelper.h"
#include "Apple/AppleAsyncTask.h"
#include "Apple/AppleStringUtils.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "Interfaces/OnlineIdentityInterface.h"

THIRD_PARTY_INCLUDES_START
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
#import <FBSDKLoginKit/FBSDKLoginKit.h>
THIRD_PARTY_INCLUDES_END


@implementation FFacebookHelper

- (id)initWithOwner:(TWeakPtr<FIOSFacebookNotificationDelegate>) InOwner
{
    self = [super init];
    
    Owner = InOwner;

    NSLog(@"Facebook SDK Version: %@", [[FBSDKSettings sharedSettings] sdkVersion]);

    [FBSDKProfile setIsUpdatedWithAccessTokenChange:YES];

    NSNotificationCenter* center = [NSNotificationCenter defaultCenter];
    [center addObserver:self selector:@selector(tokenChangeCallback:) name: FBSDKAccessTokenDidChangeNotification object:nil];
    [center addObserver:self selector:@selector(userIdChangeCallback:) name: FBSDKAccessTokenDidChangeUserIDKey object:nil];
    [center addObserver:self selector:@selector(profileChangeCallback:) name: FBSDKProfileDidChangeNotification object:nil];

    return self;
}

-(void)Shutdown
{
    [[NSNotificationCenter defaultCenter] removeObserver:self];
    Owner = nullptr;
}

/** Delegate fired when a token change has occurred */
-(void)tokenChangeCallback: (NSNotification*) note
{
    const FString UserId([FBSDKAccessToken currentAccessToken].userID);
    const FString Token([FBSDKAccessToken currentAccessToken].tokenString);
    UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Facebook Token Change UserId: %s Token: %s"), *UserId, *Token);

#if !UE_BUILD_SHIPPING
    for(NSString *key in [[note userInfo] allKeys])
    {
        NSLog(@"Key: %@ Value: %@", key, [[note userInfo] objectForKey:key]);
    }
#endif

    // header mentions FBSDKAccessTokenChangeOldKey FBSDKAccessTokenChangeNewKey
    NSNumber* DidChange = [[note userInfo] objectForKey:@"FBSDKAccessTokenDidChangeUserID"];
    bool bDidTokenChange = [DidChange boolValue];
    FBSDKAccessToken* NewToken = [[note userInfo] objectForKey:@"FBSDKAccessToken"];
    FBSDKAccessToken* OldToken = [[note userInfo] objectForKey:@"FBSDKAccessTokenOld"];

    TWeakPtr<FIOSFacebookNotificationDelegate> CapturedOwner = Owner;
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        // Notify on the game thread
        if(auto PinnedOwner = CapturedOwner.Pin())
        {
            PinnedOwner->OnFacebookTokenChange(OldToken, NewToken);
        }
        return true;
    }];
}

/** Delegate fired when a user id change has occurred */
-(void)userIdChangeCallback: (NSNotification*) note
{
    const FString UserId([FBSDKAccessToken currentAccessToken].userID);
    const FString Token([FBSDKAccessToken currentAccessToken].tokenString);
    UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Facebook UserId Change UserId: %s Token: %s"), *UserId, *Token);

#if !UE_BUILD_SHIPPING
    for(NSString *key in [[note userInfo] allKeys])
    {
        NSLog(@"Key: %@ Value: %@", key, [[note userInfo] objectForKey:key]);
    }
#endif

    TWeakPtr<FIOSFacebookNotificationDelegate> CapturedOwner = Owner;
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        // Notify on the game thread
        if(auto PinnedOwner = CapturedOwner.Pin())
        {
            PinnedOwner->OnFacebookUserIdChange();
        }
        return true;
    }];

}

/** Delegate fired when a profile change has occurred */
-(void)profileChangeCallback: (NSNotification*) note
{
    const FString UserId([FBSDKAccessToken currentAccessToken].userID);
    const FString Token([FBSDKAccessToken currentAccessToken].tokenString);
    UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Facebook Profile Change UserId: %s Token: %s"), *UserId, *Token);

#if !UE_BUILD_SHIPPING
    for(NSString *key in [[note userInfo] allKeys])
    {
        NSLog(@"Key: %@ Value: %@", key, [[note userInfo] objectForKey:key]);
    }
#endif

    // header mentions FBSDKProfileChangeOldKey FBSDKProfileChangeNewKey
    FBSDKProfile* NewProfile = [[note userInfo] objectForKey:@"FBSDKProfileNew"];
    FBSDKProfile* OldProfile = [[note userInfo] objectForKey:@"FBSDKProfileOld"];

    TWeakPtr<FIOSFacebookNotificationDelegate> CapturedOwner = Owner;
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
    {
        // Notify on the game thread
        if(auto PinnedOwner = CapturedOwner.Pin())
        {
            PinnedOwner->OnFacebookProfileChange(OldProfile, NewProfile);
        }
        return true;
    }];
}

@end
