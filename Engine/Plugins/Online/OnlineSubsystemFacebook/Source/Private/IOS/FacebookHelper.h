// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

THIRD_PARTY_INCLUDES_START
#import <Foundation/NSObject.h>
THIRD_PARTY_INCLUDES_END

@class FBSDKAccessToken;
@class FBSDKProfile;

class FIOSFacebookNotificationDelegate
{
public:
    /**
     * Callback from the SDK that the FBSDKAccessToken has changed
     *
     * @param OldToken previous access token, possibly null
     * @param NewToken current access token, possibly null
     */
    virtual void OnFacebookTokenChange(FBSDKAccessToken* OldToken, FBSDKAccessToken* NewToken) = 0;

    /**
     * Callback from the SDK when the UserId has changed
     */
    virtual void OnFacebookUserIdChange() = 0;

    /**
     * Callback from the SDK when the FBSDKProfile data has changed
     *
     * @param OldProfile previous profile, possibly null
     * @param NewProfile current profile, possibly null
     */
    virtual void OnFacebookProfileChange(FBSDKProfile* OldProfile, FBSDKProfile* NewProfile) = 0;

    virtual ~FIOSFacebookNotificationDelegate() = default;
};

/**
 * ObjC helper for communicating with the Facebook SDK, listens for events
 */
@interface FFacebookHelper : NSObject
{
    TWeakPtr<FIOSFacebookNotificationDelegate> Owner;
};

- (id)initWithOwner:(TWeakPtr<FIOSFacebookNotificationDelegate>) InOwner;
-(void)Shutdown;
@end


