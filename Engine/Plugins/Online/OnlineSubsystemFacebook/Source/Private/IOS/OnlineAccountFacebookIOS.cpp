// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineAccountFacebookIOS.h"

THIRD_PARTY_INCLUDES_START
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
THIRD_PARTY_INCLUDES_END

// FOnlineUserFacebook

FUserOnlineAccountFacebookIOS::FUserOnlineAccountFacebookIOS(FBSDKProfile* FromProfile)
{
	UserId = FString(FromProfile.userID);
	UserIdPtr = FUniqueNetIdFacebook::Create(UserId);
	RealName = FString(FromProfile.name);
	FirstName = FString(FromProfile.firstName);
	LastName = FString(FromProfile.lastName);
	Picture.PictureData.PictureURL = FString(FromProfile.imageURL.absoluteString);
	
	AuthToken = FString(FBSDKAuthenticationToken.currentAuthenticationToken.tokenString);
}

FUserOnlineAccountFacebookIOS::~FUserOnlineAccountFacebookIOS()
{

}
