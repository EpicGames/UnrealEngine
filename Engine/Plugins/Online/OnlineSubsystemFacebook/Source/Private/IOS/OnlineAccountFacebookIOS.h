// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
 
// Module includes
#include "OnlineAccountFacebookCommon.h"

@class FBSDKProfile;

/**
 * IOS specialization to support limited login
 */
class FUserOnlineAccountFacebookIOS : public FUserOnlineAccountFacebookCommon
{

public:
	using FUserOnlineAccountFacebookCommon::FUserOnlineAccountFacebookCommon;

	// FOnlineAccountFacebookIOS
	/**
	 * Create from FBSDKProfile when using limited login
	 */
	FUserOnlineAccountFacebookIOS(FBSDKProfile* FromProfile);
	
	/**
	 * Default destructor
	 */
	virtual ~FUserOnlineAccountFacebookIOS();

};
