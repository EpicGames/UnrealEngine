// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFriendsFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebook.h"

THIRD_PARTY_INCLUDES_START
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
THIRD_PARTY_INCLUDES_END

FOnlineFriendsFacebook::FOnlineFriendsFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineFriendsFacebookCommon(InSubsystem)
{
}

FOnlineFriendsFacebook::~FOnlineFriendsFacebook()
{
}

bool FOnlineFriendsFacebook::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate)
{
	TSharedPtr<FOnlineIdentityFacebook> Identity = StaticCastSharedPtr<FOnlineIdentityFacebook>(FacebookSubsystem->GetIdentityInterface());
	if (Identity->IsUsingClassicLogin())
	{
		return FOnlineFriendsFacebookCommon::ReadFriendsList(LocalUserNum, ListName, Delegate);
	}
	else
	{
		FString ErrorStr;
		if (Identity->GetLoginStatus(LocalUserNum) != ELoginStatus::LoggedIn)
		{
			ErrorStr = FString::Printf(TEXT("User LocalUserNum=%d not logged in."), LocalUserNum);
		}
		else if (!ListName.Equals(EFriendsLists::ToString(EFriendsLists::Default), ESearchCase::IgnoreCase))
		{
			// wrong list type
			ErrorStr = TEXT("Only the default friends list is supported");
		}
		else if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
		{
			// invalid local player index
			ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
		}
		
		if (ErrorStr.IsEmpty())
		{
			FOnlineFriendsList& FriendsList = FriendsMap.FindOrAdd(LocalUserNum);
			FriendsList.Friends.Empty();
			
			FriendsList.Friends.Reserve([FBSDKProfile.currentProfile.friendIDs count]);
			for ( NSString* FriendId in FBSDKProfile.currentProfile.friendIDs)
			{
				TSharedRef<FOnlineFriendFacebook> FriendEntry = MakeShared<FOnlineFriendFacebook>(FString(FriendId));
				FriendsList.Friends.Add(FriendEntry);
			}
		}
		Delegate.ExecuteIfBound(LocalUserNum, ErrorStr.IsEmpty(), EFriendsLists::ToString(EFriendsLists::Default), ErrorStr);
		return ErrorStr.IsEmpty();
	}
}
