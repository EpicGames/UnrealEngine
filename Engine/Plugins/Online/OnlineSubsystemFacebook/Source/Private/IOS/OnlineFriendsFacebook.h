// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineFriendsFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/**
 * Facebook service implementation of the online friends interface (iOS)
 */
class FOnlineFriendsFacebook :
	public FOnlineFriendsFacebookCommon
{

PACKAGE_SCOPE:

	/**
	 * Constructor
	 *
	 * @param InSubsystem Facebook subsystem being used
	 */
	FOnlineFriendsFacebook(FOnlineSubsystemFacebook* InSubsystem);

public:
	
	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendsFacebook();

	// IOnlineFriends
	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;

private:

	/**
	 * Should use the OSS param constructor instead
	 */
	FOnlineFriendsFacebook();

};

typedef TSharedPtr<FOnlineFriendsFacebook, ESPMode::ThreadSafe> FOnlineFriendsFacebookPtr;
