// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineIdentityFacebook.h"
#include "Apple/AppleAsyncTask.h"
#include "Apple/AppleStringUtils.h"
#include "OnlineSharingFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "Interfaces/OnlineSharingInterface.h"
#include "Interfaces/OnlineExternalUIInterface.h"

#include "Misc/ConfigCacheIni.h"
#include "IOS/IOSAsyncTask.h"

THIRD_PARTY_INCLUDES_START
#if UE_WITH_CLASSIC_FACEBOOK_LOGIN
#import <AppTrackingTransparency/AppTrackingTransparency.h>
#endif
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
#import <FBSDKLoginKit/FBSDKLoginKit-Swift.h>
THIRD_PARTY_INCLUDES_END

///////////////////////////////////////////////////////////////////////////////////////
// FOnlineIdentityFacebook implementation

static TOptional<bool> ShouldUseClassicLogin()
{
	bool bUseClassicLogin = true;
	bool bFallbackToLimitedLogin = false;

	GConfig->GetBool(TEXT("OnlineSubsystemFacebook"), TEXT("bUseClassicLogin"), bUseClassicLogin, GEngineIni);
	GConfig->GetBool(TEXT("OnlineSubsystemFacebook"), TEXT("bFallbackToLimitedLogin"), bFallbackToLimitedLogin, GEngineIni);

	bool bClassicLoginAllowed = false;
#if UE_WITH_CLASSIC_FACEBOOK_LOGIN
	switch(ATTrackingManager.trackingAuthorizationStatus)
	{
		case ATTrackingManagerAuthorizationStatusAuthorized:
			bClassicLoginAllowed = true;
			break;
		case ATTrackingManagerAuthorizationStatusDenied:
		case ATTrackingManagerAuthorizationStatusNotDetermined:
		case ATTrackingManagerAuthorizationStatusRestricted:
			bClassicLoginAllowed = false;
			break;
	}
#endif

	if (bUseClassicLogin && !bClassicLoginAllowed)
	{
		if (bFallbackToLimitedLogin)
		{
			bUseClassicLogin = false;
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Falling back to Limited Facebook login because application tracking was not authorized"));
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Error, TEXT("Classic Facebook login is not supported if application tracking was not authorized"));
			return NullOpt;
		}
	}
	return bUseClassicLogin;
}

FOnlineIdentityFacebook::FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineIdentityFacebookCommon(InSubsystem)
{
	// Setup scopes fields
	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("ScopeFields"), ScopeFields, GEngineIni);
	// always required fields
	ScopeFields.AddUnique(TEXT(PERM_PUBLIC_PROFILE));

}

void FOnlineIdentityFacebook::Init()
{
    FacebookHelper = [[FFacebookHelper alloc] initWithOwner: AsShared()];
}

void FOnlineIdentityFacebook::Shutdown()
{
    [FacebookHelper Shutdown];
	FacebookHelper = nil;
}

void FOnlineIdentityFacebook::OnFacebookTokenChange(FBSDKAccessToken* OldToken, FBSDKAccessToken* NewToken)
{
	UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentityFacebook::OnFacebookTokenChange Old: %p New: %p"), OldToken, NewToken);
}

void FOnlineIdentityFacebook::OnFacebookUserIdChange()
{
	UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentityFacebook::OnFacebookUserIdChange"));
}

void FOnlineIdentityFacebook::OnFacebookProfileChange(FBSDKProfile* OldProfile, FBSDKProfile* NewProfile)
{
	UE_LOG_ONLINE_IDENTITY(Warning, TEXT("FOnlineIdentityFacebook::OnFacebookProfileChange Old: %p New: %p"), OldProfile, NewProfile);
}

bool FOnlineIdentityFacebook::IsUsingClassicLogin() const 
{
	return bIsUsingClassicLogin;
}

bool FOnlineIdentityFacebook::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	bool bTriggeredLogin = true;

	if (bIsLoginInProgress)
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdFacebook::EmptyId(), TEXT("Login already in progress"));
		return false;
	}

	if (GetLoginStatus(LocalUserNum) != ELoginStatus::NotLoggedIn)
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, true, *GetUniquePlayerId(LocalUserNum), TEXT("Already logged in"));
		return false;
	}

	TOptional<bool> UseClassicLogin = ShouldUseClassicLogin();
	if (!UseClassicLogin.IsSet())
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdFacebook::EmptyId(), TEXT("Login type unsupported"));
		return false;
	}

	bIsLoginInProgress = true;
	bIsUsingClassicLogin = *UseClassicLogin;
	
	dispatch_async(dispatch_get_main_queue(),^
		{
			if ((bIsUsingClassicLogin && (FBSDKAccessToken.currentAccessToken == nil || [FBSDKAccessToken.currentAccessToken isExpired])) ||
				(!bIsUsingClassicLogin && FBSDKProfile.currentProfile == nil) )
			{
				FBSDKLoginManager* loginManager = [[FBSDKLoginManager alloc] init];
				NSMutableArray* Permissions = [[NSMutableArray alloc] initWithCapacity:ScopeFields.Num()];
				for (int32 ScopeIdx = 0; ScopeIdx < ScopeFields.Num(); ScopeIdx++)
				{
					NSString* ScopeStr = [NSString stringWithFString:ScopeFields[ScopeIdx]];
					[Permissions addObject: ScopeStr];
				}

				FBSDKLoginConfiguration *Configuration = [[FBSDKLoginConfiguration alloc] initWithPermissions: Permissions
																									 tracking: bIsUsingClassicLogin? FBSDKLoginTrackingEnabled : FBSDKLoginTrackingLimited];
				
				[loginManager logInFromViewController: nil
					configuration: Configuration
					completion: ^(FBSDKLoginManagerLoginResult* result, NSError* error)
					{
						UE_LOG_ONLINE_IDENTITY(Display, TEXT("[FBSDKLoginManager logInFromViewController]"));
						bool bSuccessfulLogin = false;

						FString ErrorStr;
						if(error)
						{
							ErrorStr = FString::Printf(TEXT("[%lld] %s"), [error code], *FString([error localizedDescription]));
							UE_LOG_ONLINE_IDENTITY(Display, TEXT("[FBSDKLoginManager logInFromViewController = %s]"), *ErrorStr);

						}
						else if(result.isCancelled)
						{
							ErrorStr = LOGIN_CANCELLED;
							UE_LOG_ONLINE_IDENTITY(Display, TEXT("[FBSDKLoginManager logInFromViewController = cancelled"));
						}
						else
						{
							UE_LOG_ONLINE_IDENTITY(Display, TEXT("[FBSDKLoginManager logInFromViewController = true]"));
							bSuccessfulLogin = true;
						}

                        TArray<FString> GrantedPermissions, DeclinedPermissions;

                        GrantedPermissions.Reserve(result.grantedPermissions.count);
                        for(NSString* permission in result.grantedPermissions)
                        {
                            GrantedPermissions.Add(permission);
                        }

                        DeclinedPermissions.Reserve(result.declinedPermissions.count);
                        for(NSString* permission in result.declinedPermissions)
                        {
                            DeclinedPermissions.Add(permission);
                        }

						[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
						{
							// Trigger this on the game thread
							if (bSuccessfulLogin)
							{
								TSharedPtr<FOnlineSharingFacebook> Sharing = StaticCastSharedPtr<FOnlineSharingFacebook>(FacebookSubsystem->GetSharingInterface());
								if (bIsUsingClassicLogin)
								{
									const FString AccessToken([[result token] tokenString]);
									if (Sharing)
									{
										Sharing->SetCurrentPermissions(GrantedPermissions, DeclinedPermissions);
									}
									Login(LocalUserNum, AccessToken);
								}
								else
								{
									if (Sharing)
									{
										Sharing->SetCurrentPermissions(GrantedPermissions, DeclinedPermissions);
									}
									LoginLimited(LocalUserNum);
								}
							}
							else
							{
								OnLoginAttemptComplete(LocalUserNum, false, ErrorStr);
							}

							return true;
						 }];
					}
				];
			}
			else
			{
				// Skip right to attempting to use the token to query user profile
				// or the current profile in case of limited login
				// Could fail with an expired auth token (eg. user revoked app)

				if (bIsUsingClassicLogin)
				{
					const FString AccessToken([FBSDKAccessToken.currentAccessToken tokenString]);
					[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
					 {
						Login(LocalUserNum, FString(AccessToken));
						return true;
					}];
				}
				else
				{
					[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
					 {
						LoginLimited(LocalUserNum);
						return true;
					}];
				}
			}
		}
	);

	return bTriggeredLogin;	
}

void FOnlineIdentityFacebook::Login(int32 LocalUserNum, const FString& AccessToken)
{
	FOnProfileRequestComplete CompletionDelegate = FOnProfileRequestComplete::CreateLambda([this](int32 LocalUserNumFromRequest, bool bWasProfileRequestSuccessful, const FString& ErrorStr)
	{
		OnLoginAttemptComplete(LocalUserNumFromRequest, bWasProfileRequestSuccessful, ErrorStr);
	});

	ProfileRequest(LocalUserNum, AccessToken, ProfileFields, CompletionDelegate);
}

void FOnlineIdentityFacebook::LoginLimited(int32 LocalUserNum)
{
	// Gather data from profile snapshot and store it
	TSharedRef<FUserOnlineAccountFacebookIOS> User = MakeShared<FUserOnlineAccountFacebookIOS>(FBSDKProfile.currentProfile);
	UserAccounts.Add(User->GetUserId()->ToString(), User);
	UserIds.Add(LocalUserNum, User->GetUserId());
	
	OnLoginAttemptComplete(LocalUserNum, true, TEXT(""));
}

FString FOnlineIdentityFacebook::GetAuthToken(int32 LocalUserNum) const
{
	if (IsUsingClassicLogin())
	{
		return FOnlineIdentityFacebookCommon::GetAuthToken(LocalUserNum);
	}
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			FString AuthToken;
			if (UserAccount->GetAuthAttribute(AUTH_ATTR_ID_TOKEN, AuthToken))
			{
				return AuthToken;
			}
		}
	}
	return FString();
}

void FOnlineIdentityFacebook::OnLoginAttemptComplete(int32 LocalUserNum, bool bSucceeded, const FString& ErrorStr)
{
	LoginStatus = bSucceeded ? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn;
	if (LoginStatus == ELoginStatus::LoggedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Display, TEXT("Facebook login was successful"));
		FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
		check(UserId.IsValid());
		bIsLoginInProgress = false;
		TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, ErrorStr);
		TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *UserId);
	}
	else
	{
		const FString NewErrorStr(ErrorStr);
		// Clean up anything left behind from cached access tokens
		dispatch_async(dispatch_get_main_queue(),^
		{
			FBSDKLoginManager* loginManager = [[FBSDKLoginManager alloc] init];
			[loginManager logOut];

			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			 {
				// Trigger this on the game thread
				UE_LOG_ONLINE_IDENTITY(Display, TEXT("Facebook login failed: %s"), *NewErrorStr);

				FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
				if (UserId.IsValid())
				{
					// remove cached user account
					UserAccounts.Remove(UserId->ToString());
				}
				else
				{
					UserId = FUniqueNetIdFacebook::EmptyId();
				}
				// remove cached user id
				UserIds.Remove(LocalUserNum);

				bIsLoginInProgress = false;
				TriggerOnLoginCompleteDelegates(LocalUserNum, false, *UserId, NewErrorStr);
				return true;
			 }];
		});
	}
}

bool FOnlineIdentityFacebook::Logout(int32 LocalUserNum)
{
	if (LoginStatus == ELoginStatus::LoggedIn)
	{
		dispatch_async(dispatch_get_main_queue(),^
		{
			FBSDKLoginManager* loginManager = [[FBSDKLoginManager alloc] init];
			[loginManager logOut];

			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				// Trigger this on the game thread
				FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
				if (UserId.IsValid())
				{
					// remove cached user account
					UserAccounts.Remove(UserId->ToString());
				}
				else
				{
					UserId = FUniqueNetIdFacebook::EmptyId();
				}
				// remove cached user id
				UserIds.Remove(LocalUserNum);

				FacebookSubsystem->ExecuteNextTick([this, UserId, LocalUserNum]() {
					LoginStatus = ELoginStatus::NotLoggedIn;
					TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
					TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
				});
				return true;
			 }];
		});
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		FacebookSubsystem->ExecuteNextTick([this, LocalUserNum](){
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}

	return true;
}


