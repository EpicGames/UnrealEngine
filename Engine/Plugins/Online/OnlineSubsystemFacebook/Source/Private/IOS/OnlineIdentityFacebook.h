// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
// Module includes
#include "OnlineIdentityFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"
#include "OnlineAccountFacebookIOS.h"

#import "FacebookHelper.h"

class FUserOnlineAccountFacebookIOS;
class FOnlineSubsystemFacebook;

@class FBSDKAccessToken;
@class FBSDKProfile;
@class FFacebookHelper;

/** iOS implementation of a Facebook user account */
using FUserOnlineAccountFacebook  = FUserOnlineAccountFacebookIOS;

/**
 * Facebook service implementation of the online identity interface
 */
class FOnlineIdentityFacebook :
	public FOnlineIdentityFacebookCommon,
    public FIOSFacebookNotificationDelegate,
    public TSharedFromThis<FOnlineIdentityFacebook>
{

public:

	//~ Begin IOnlineIdentity Interface	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	//~ End IOnlineIdentity Interface

public:

	/**
	 * Default constructor
	 */
	FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityFacebook()
	{
	}

PACKAGE_SCOPE:

    /** Inits the ObjC bridge */
    void Init();

	/** Shutdown the interface */
	void Shutdown();

	/**
	 * Login user to Facebook using classic login, given a valid access token
	 *
	 * @param LocalUserNum local id of the requesting user
	 * @param AccessToken opaque Facebook supplied access token
	 */
	void Login(int32 LocalUserNum, const FString& AccessToken);

	/**
	 * Gathers login information from limited login token
	 *
	 * @param LocalUserNum local id of the requesting user
	 */
	void LoginLimited(int32 LocalUserNum);

	/**
	 * Returns whether we are using classic login or limited login
	 *
	 * @return true if using classic login. false if using limited login
	 */
	bool IsUsingClassicLogin() const;
private:

	/**
	 * Generic callback for all attempts at login, called to end the attempt
	 *
	 * @param local id of the requesting user
	 * @param bSucceeded was login succesful?
	 * @param ErrorStr any error as a result of the login attempt
	 */
	void OnLoginAttemptComplete(int32 LocalUserNum, bool bSucceeded, const FString& ErrorStr);

    virtual void OnFacebookTokenChange(FBSDKAccessToken* OldToken, FBSDKAccessToken* NewToken) override;
    virtual void OnFacebookUserIdChange() override;
    virtual void OnFacebookProfileChange(FBSDKProfile* OldProfile, FBSDKProfile* NewProfile) override;

    /** ObjC helper for access to SDK methods and callbacks */
	FFacebookHelper* FacebookHelper;

	/** The current state of our login */
	ELoginStatus::Type LoginStatus = ELoginStatus::NotLoggedIn;

	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;
	
	/** Did we log in using limited login? */
	bool bIsUsingClassicLogin = false;
	
	/** Did we started a login but didn't finished yet? */
	bool bIsLoginInProgress = false;
	 
};

typedef TSharedPtr<FOnlineIdentityFacebook, ESPMode::ThreadSafe> FOnlineIdentityFacebookPtr;
