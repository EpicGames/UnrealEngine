// Copyright Epic Games, Inc. All Rights Reserved.


// Module includes
#include "OnlineSharingFacebook.h"
#include "Apple/AppleAsyncTask.h"
#include "Apple/AppleStringUtils.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebook.h"

#include "IOS/IOSAsyncTask.h"

THIRD_PARTY_INCLUDES_START
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
#import <FBSDKLoginKit/FBSDKLoginKit-Swift.h>
THIRD_PARTY_INCLUDES_END

FOnlineSharingFacebook::FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineSharingFacebookCommon(InSubsystem)
{
}

FOnlineSharingFacebook::~FOnlineSharingFacebook()
{
}

void FOnlineSharingFacebook::RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& CompletionDelegate)
{
	TSharedPtr<FOnlineIdentityFacebook> IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(Subsystem->GetIdentityInterface());
	if (IdentityInt.IsValid() && IdentityInt->IsUsingClassicLogin())
	{
		FOnlineSharingFacebookCommon::RequestCurrentPermissions(LocalUserNum, CompletionDelegate);
	}
	else
	{
		bool bSuccess = FBSDKProfile.currentProfile != nil;
		if (bSuccess)
		{
			TArray<FString> GrantedPermissions;
			
			for(NSString* Permission in FBSDKProfile.currentProfile.permissions)
			{
				GrantedPermissions.Add(FString(Permission));
			}
			
			SetCurrentPermissions(GrantedPermissions, TArray<FString>{});
		}
		TArray<FSharingPermission> StoredPermissions;
		GetCurrentPermissions(LocalUserNum, StoredPermissions);
		CompletionDelegate.ExecuteIfBound(LocalUserNum, bSuccess, StoredPermissions);
	}
}

bool FOnlineSharingFacebook::RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions)
{
	bool bTriggeredRequest = false;

	ensure((NewPermissions & ~EOnlineSharingCategory::ReadPermissionMask) == EOnlineSharingCategory::None);

	TSharedPtr<FOnlineIdentityFacebook> IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(Subsystem->GetIdentityInterface());
	if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		bTriggeredRequest = true;

		dispatch_async(dispatch_get_main_queue(),^ 
			{
				// Fill in an nsarray with the permissions which match those that the user has set,
				// Here we iterate over each category, adding each individual permission linked with it in the ::SetupPermissionMaps

				TArray<FSharingPermission> PermissionsNeeded;
				const bool bHasPermission = CurrentPermissions.HasPermission(NewPermissions, PermissionsNeeded);
				if (!bHasPermission)
				{
                    NSMutableArray* PermissionsRequested = [[NSMutableArray alloc] init];
					for (const FSharingPermission& Permission : PermissionsNeeded)
					{
						[PermissionsRequested addObject:[NSString stringWithFString:Permission.Name]];
					}

					FBSDKLoginConfiguration *Configuration = [[FBSDKLoginConfiguration alloc] initWithPermissions: PermissionsRequested
																										 tracking: IdentityInt->IsUsingClassicLogin()? FBSDKLoginTrackingEnabled : FBSDKLoginTrackingLimited];

                    FBSDKLoginManager *loginManager = [[FBSDKLoginManager alloc] init];
                    [loginManager logInFromViewController: nil
											configuration: Configuration
											   completion: ^(FBSDKLoginManagerLoginResult* Result, NSError* Error)
                        {
                            UE_LOG_ONLINE_SHARING(Display, TEXT("logInFromViewController : Success - %d"), Error == nil);
                            [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
                            {
                                if (Error == nil)
                                {
                                    FOnRequestCurrentPermissionsComplete PermsDelegate = FOnRequestCurrentPermissionsComplete::CreateRaw(this, &FOnlineSharingFacebook::OnRequestCurrentReadPermissionsComplete);
                                    RequestCurrentPermissions(LocalUserNum, PermsDelegate);
                                }
                                else
                                {
                                    TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
                                }
                                return true;
                            }];
                        }
                    ];
                }
                else
                {
                    // All permissions were already granted, no need to reauthorize
                    TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, true);
                }
			}
		);
	}
	else
	{
		// If we weren't logged into Facebook we cannot do this action
		TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
	}

	// We did kick off a request
	return bTriggeredRequest;
}

void FOnlineSharingFacebook::OnRequestCurrentReadPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& Permissions)
{
	TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, bWasSuccessful);
}

bool FOnlineSharingFacebook::RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy)
{
	bool bTriggeredRequest = false;

	ensure((NewPermissions & ~EOnlineSharingCategory::PublishPermissionMask) == EOnlineSharingCategory::None);
	
	TSharedPtr<FOnlineIdentityFacebook> IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(Subsystem->GetIdentityInterface());
	if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		bTriggeredRequest = true;

		dispatch_async(dispatch_get_main_queue(),^ 
			{
				// Fill in an nsarray with the permissions which match those that the user has set,
				// Here we iterate over each category, adding each individual permission linked with it in the ::SetupPermissionMaps

				TArray<FSharingPermission> PermissionsNeeded;
				const bool bHasPermission = CurrentPermissions.HasPermission(NewPermissions, PermissionsNeeded);
				if (!bHasPermission)
				{
                    NSMutableArray* PermissionsRequested = [[NSMutableArray alloc] init];
					for (const FSharingPermission& Permission : PermissionsNeeded)
					{
						[PermissionsRequested addObject:[NSString stringWithFString:Permission.Name]];
					}

					FBSDKLoginConfiguration *Configuration = [[FBSDKLoginConfiguration alloc] initWithPermissions: PermissionsRequested
																										 tracking: IdentityInt->IsUsingClassicLogin()? FBSDKLoginTrackingEnabled : FBSDKLoginTrackingLimited];

					FBSDKLoginManager *loginManager = [[FBSDKLoginManager alloc] init];
					[loginManager logInFromViewController: nil
											configuration: Configuration
											   completion: ^(FBSDKLoginManagerLoginResult* Result, NSError* Error)
						{
                            UE_LOG_ONLINE_SHARING(Display, TEXT("logInWithPublishPermissions : Success - %d"), Error == nil);
                            [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
                            {
                                if (Error == nil)
                                {
                                    FOnRequestCurrentPermissionsComplete PermsDelegate = FOnRequestCurrentPermissionsComplete::CreateRaw(this, &FOnlineSharingFacebook::OnRequestCurrentPublishPermissionsComplete);
                                    RequestCurrentPermissions(LocalUserNum, PermsDelegate);
                                }
                                else
                                {
                                    TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
                                }
                                return true;
                            }];
                        }
                     ];
                }
                else
                {
                    // All permissions were already granted, no need to reauthorize
                    TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, true);
                }
			}
		);
	}
	else
	{
		// If we weren't logged into Facebook we cannot do this action
		TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
	}

	// We did kick off a request
	return bTriggeredRequest;
}

void FOnlineSharingFacebook::OnRequestCurrentPublishPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& Permissions)
{
	TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, bWasSuccessful);
}
