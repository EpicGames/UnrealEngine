// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
// Module includes
#include "OnlineSharingFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/**
 * Facebook implementation of the Online Sharing Interface
 */
class FOnlineSharingFacebook : public FOnlineSharingFacebookCommon
{

public:

	//~ Begin IOnlineSharing Interface
	virtual bool RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions) override;
	virtual bool RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy) override;
	//~ End IOnlineSharing Interface

public:

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineSharingFacebook();

	// IOnlineSharing
	virtual void RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& CompletionDelegate) override;

private:

	/**
	 * Delegate fired when current permissions have been updated after a read permissions request
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true permission query was successful
	 * @param Permissions set of all known permissions
	 */
	void OnRequestCurrentReadPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& Permissions);

	/**
	 * Delegate fired when current permissions have been updated after a publish permissions request
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param bWasSuccessful true permission query was successful
	 * @param Permissions set of all known permissions
	 */
	void OnRequestCurrentPublishPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& Permissions);

};


typedef TSharedPtr<FOnlineSharingFacebook, ESPMode::ThreadSafe> FOnlineSharingFacebookPtr;
