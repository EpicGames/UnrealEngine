// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemFacebook.h"
#include "Apple/AppleAsyncTask.h"
#include "Apple/AppleStringUtils.h"
#include "OnlineSubsystemFacebookPrivate.h"

#include "Misc/CoreDelegates.h"
#include "IOS/IOSAppDelegate.h"
#include "Misc/ConfigCacheIni.h"

#include "OnlineExternalUIInterfaceFacebook.h"
#include "OnlineFriendsFacebook.h"
#include "OnlineIdentityFacebook.h"
#include "OnlineSharingFacebook.h"
#include "OnlineUserFacebook.h"

THIRD_PARTY_INCLUDES_START
#import <AuthenticationServices/AuthenticationServices.h>
#import <SafariServices/SafariServices.h>
#import <FBSDKCoreKit/FBSDKCoreKit-Swift.h>
THIRD_PARTY_INCLUDES_END

#define FACEBOOK_DEBUG_ENABLED 0

FOnlineSubsystemFacebook::FOnlineSubsystemFacebook(FName InInstanceName)
	: FOnlineSubsystemFacebookCommon(InInstanceName)
{
}

FOnlineSubsystemFacebook::~FOnlineSubsystemFacebook()
{
}

static void OnFacebookOpenURL(UIApplication* application, NSURL* url, NSString* sourceApplication, id annotation)
{
	[[FBSDKApplicationDelegate sharedInstance] application:application
		openURL : url
		sourceApplication : sourceApplication
		annotation : annotation];
}

static void OnFacebookAppDidBecomeActive()
{
#if 0 // turn off analytics
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[[FBSDKAppEvents shared] activateApp];
	});
#endif
}

/** Add verbose logging for various Facebook SDK features */
void SetFBLoggingBehavior()
{
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorAppEvents];
#if FACEBOOK_DEBUG_ENABLED
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorAccessTokens];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorPerformanceCharacteristics];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorAppEvents];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorInformational];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorCacheErrors];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorUIControlErrors];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorGraphAPIDebugWarning];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorGraphAPIDebugInfo];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorNetworkRequests];
	[[FBSDKSettings sharedSettings] enableLoggingBehavior:FBSDKLoggingBehaviorDeveloperErrors];
#endif
}

/** Print various details about the Facebook SDK */
void PrintSDKStatus()
{
	NSString* AppId = [[FBSDKSettings sharedSettings] appID];
	NSString* SDKVersion = [[FBSDKSettings sharedSettings] sdkVersion];
	NSString* GraphVer = [[FBSDKSettings sharedSettings] graphAPIVersion];
	NSString* OverrideAppId = [[FBSDKAppEvents shared] loggingOverrideAppID];
    NSSet* LoggingBehaviors = [[FBSDKSettings sharedSettings] loggingBehaviors];

	UE_LOG_ONLINE(Log, TEXT("Facebook SDK:%s"), *FString(SDKVersion));
	UE_LOG_ONLINE(Log, TEXT("AppId:%s"), *FString(AppId));
	UE_LOG_ONLINE(Log, TEXT("OverrideAppId:%s"), *FString(OverrideAppId));
	UE_LOG_ONLINE(Log, TEXT("GraphVer:%s"), *FString(GraphVer));

	if (LoggingBehaviors != nil && [LoggingBehaviors count] > 0)
	{
		UE_LOG_ONLINE(Verbose, TEXT("Logging:"));
		for (NSString* loggingBehavior in LoggingBehaviors)
		{
			UE_LOG_ONLINE(Verbose, TEXT(" - %s"), *FString(loggingBehavior));
		}
	}
}

bool FOnlineSubsystemFacebook::Init()
{
	if (!FOnlineSubsystemFacebookCommon::Init())
	{
        return false;
    }
    FIOSCoreDelegates::OnOpenURL.AddStatic(&OnFacebookOpenURL);
    FCoreDelegates::ApplicationHasReactivatedDelegate.AddStatic(&OnFacebookAppDidBecomeActive);

    auto IosIdentity = MakeShared<FOnlineIdentityFacebook>(this);
    IosIdentity->Init();

    FacebookIdentity = MoveTemp(IosIdentity);
    FacebookSharing = MakeShared<FOnlineSharingFacebook>(this);
    FacebookFriends = MakeShared<FOnlineFriendsFacebook>(this);
    FacebookUser = MakeShared<FOnlineUserFacebook>(this);
	FacebookExternalUI = MakeShared<FOnlineExternalUIFacebook>(this);

    FString AnalyticsId;
    GConfig->GetString(TEXT("OnlineSubsystemFacebook"), TEXT("AnalyticsId"), AnalyticsId, GEngineIni);

    NSString* APIVerStr = [NSString stringWithFString:GetAPIVer()];
    [[FBSDKSettings sharedSettings] setGraphAPIVersion:APIVerStr];
    SetFBLoggingBehavior();

    /** Sets whether data such as that generated through FBSDKAppEvents and sent to Facebook should be restricted from being used for other than analytics and conversions */
    [[FBSDKSettings sharedSettings] setIsEventDataUsageLimited:TRUE];

    // Trigger Facebook SDK last now that everything is setup
    dispatch_async(dispatch_get_main_queue(), ^
    {
        UIApplication* sharedApp = [UIApplication sharedApplication];
        NSDictionary* launchDict = [IOSAppDelegate GetDelegate].launchOptions;
        if ([[FBSDKSettings sharedSettings] isAdvertiserIDCollectionEnabled])
        {
            if (!AnalyticsId.IsEmpty())
            {
                NSString* AnalyticsStr = [NSString stringWithFString:AnalyticsId];
                [[FBSDKAppEvents shared] setLoggingOverrideAppID:AnalyticsStr];
            }
            [[FBSDKAppEvents shared] activateApp];
        }
        [[FBSDKApplicationDelegate sharedInstance] application:sharedApp didFinishLaunchingWithOptions: launchDict];
        PrintSDKStatus();
    });

	return true;
}

bool FOnlineSubsystemFacebook::Shutdown()
{
	StaticCastSharedPtr<FOnlineIdentityFacebook>(FacebookIdentity)->Shutdown();

	return FOnlineSubsystemFacebookCommon::Shutdown();
}

bool FOnlineSubsystemFacebook::IsEnabled() const
{
    return FOnlineSubsystemFacebookCommon::IsEnabled();
}

