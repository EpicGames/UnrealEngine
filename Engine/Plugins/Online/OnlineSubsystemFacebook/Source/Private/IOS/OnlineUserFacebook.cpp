// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineUserFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"

// FOnlineUserFacebook

FOnlineUserFacebook::FOnlineUserFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineUserFacebookCommon(InSubsystem)
{
}

FOnlineUserFacebook::~FOnlineUserFacebook()
{

}
