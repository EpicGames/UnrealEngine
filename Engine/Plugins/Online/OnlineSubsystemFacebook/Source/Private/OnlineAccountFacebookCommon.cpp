// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAccountFacebookCommon.h"
#include "OnlineSubsystemFacebookPrivate.h"

bool FUserOnlineAccountFacebookCommon::Parse(const FString& InAuthTicket, const FString& JsonStr)
{
	bool bResult = false;
	if (!InAuthTicket.IsEmpty())
	{
		if (!JsonStr.IsEmpty())
		{
			if (FromJson(JsonStr))
			{
				if (!UserId.IsEmpty())
				{
					UserIdPtr = FUniqueNetIdFacebook::Create(UserId);

					// update the access token
					AuthTicket = InAuthTicket;

					bResult = true;
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountFacebookCommon: Missing user id. payload=%s"), *JsonStr);
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountFacebookCommon: Invalid response payload=%s"), *JsonStr);
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountFacebookCommon: Empty Json string"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FUserOnlineAccountFacebookCommon: Empty auth ticket string"));
	}

	return bResult;
}

FUniqueNetIdRef FUserOnlineAccountFacebookCommon::GetUserId() const
{
	return UserIdPtr;
}

FString FUserOnlineAccountFacebookCommon::GetRealName() const
{
	return RealName;
}

FString FUserOnlineAccountFacebookCommon::GetDisplayName(const FString& Platform) const
{
	return RealName;
}

bool FUserOnlineAccountFacebookCommon::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

bool FUserOnlineAccountFacebookCommon::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	return SetAccountData(AttrName, AttrValue);
}

FString FUserOnlineAccountFacebookCommon::GetAccessToken() const
{
	return AuthTicket;
}

bool FUserOnlineAccountFacebookCommon::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	if (AttrName == AUTH_ATTR_REFRESH_TOKEN)
	{
		OutAttrValue = AuthTicket;
		return true;
	}
	
	if (AttrName == AUTH_ATTR_ID_TOKEN)
	{
		OutAttrValue = AuthToken;
		return true;
	}

	return false;
}

