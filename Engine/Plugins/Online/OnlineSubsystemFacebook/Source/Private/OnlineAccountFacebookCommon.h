// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "OnlineSubsystemFacebookTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemFacebookPackage.h"
#include "Interfaces/IHttpRequest.h"

/** Details about the logged in user */
#define ME_FIELD_ID "id"
#define ME_FIELD_NAME "name"
#define ME_FIELD_FIRSTNAME "first_name"
#define ME_FIELD_LASTNAME "last_name"
#define ME_FIELD_PICTURE "picture"

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountFacebookCommon : 
	public FUserOnlineAccount,
	public FJsonSerializable
{
public:

	// FOnlineUser
	
	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override;
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FUserOnlineAccountFacebookCommon

	explicit FUserOnlineAccountFacebookCommon(const FString& InUserId = FString(), const FString& InAuthTicket = FString())
		: UserIdPtr(FUniqueNetIdFacebook::Create(InUserId))
		, UserId(InUserId)
		, AuthTicket(InAuthTicket)
	{
	}

	virtual ~FUserOnlineAccountFacebookCommon()
	{
	}

	/**
	 * Parse Json profile request data into the user account
	 *
	 * @param InAuthTicket previously associated auth ticket with this Json response
	 * @param JsonStr a json payload from a /me request
	 */
	bool Parse(const FString& InAuthTicket, const FString& JsonStr);

protected:

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/**
	 * Set account data attribute
	 *
	 * @param Key account data entry key
	 * @param Val value that we want to set
	 *
	 * @return true if entry changed
	 */
	inline bool SetAccountData(const FString& Key, const FString& Val)
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal == NULL || *FoundVal != Val)
		{
			AccountData.Add(Key, Val);
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;
	/** Id associated with the user account provided by the online service during registration */
	FString UserId;
	/** Real name */
	FString RealName;
	/** First name */
	FString FirstName;
	/** Last name */
	FString LastName;
	/** Profile pic */
	FUserOnlineFacebookPicture Picture;
	/** Ticket which is provided to user once authenticated by the online service */
	FString AuthTicket;
	/** Authorization Token received when using limited login*/
	FString AuthToken;
	/** Any addition account data associated with the user */
	FJsonSerializableKeyValueMap AccountData;

private:

	// FJsonSerializable

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE(ME_FIELD_ID, UserId);
		JSON_SERIALIZE(ME_FIELD_NAME, RealName);
		JSON_SERIALIZE(ME_FIELD_FIRSTNAME, FirstName);
		JSON_SERIALIZE(ME_FIELD_LASTNAME, LastName);
		JSON_SERIALIZE_OBJECT_SERIALIZABLE(ME_FIELD_PICTURE, Picture);
		JSON_SERIALIZE_SIMPLECOPY(AccountData);
	END_JSON_SERIALIZER

	/** Allow the FB identity to fill in our private members from it's callbacks */
	friend class FOnlineIdentityFacebook;
};
