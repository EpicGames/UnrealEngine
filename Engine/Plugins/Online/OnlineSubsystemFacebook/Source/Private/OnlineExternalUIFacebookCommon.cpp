// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIFacebookCommon.h"
#include "OnlineSubsystemFacebook.h"

bool FOnlineExternalUIFacebookCommon::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	bool bStarted = false;

	FacebookSubsystem->ExecuteNextTick([ControllerIndex, Delegate]()
	{
		Delegate.ExecuteIfBound(nullptr, ControllerIndex, FOnlineError(EOnlineErrorResult::Unknown));
	});

	return bStarted;
}

bool FOnlineExternalUIFacebookCommon::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowAchievementsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowLeaderboardUI( const FString& LeaderboardName )
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIFacebookCommon::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	return false;
}

