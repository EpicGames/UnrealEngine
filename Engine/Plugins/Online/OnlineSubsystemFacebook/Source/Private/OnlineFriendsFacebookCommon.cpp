// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFriendsFacebookCommon.h"
#include "OnlineSubsystemFacebookPrivate.h"
#if USES_RESTFUL_FACEBOOK
#include "OnlineIdentityFacebookRest.h"
#else // USES_RESTFUL_FACEBOOK
#include "OnlineIdentityFacebook.h"
#endif // USES_RESTFUL_FACEBOOK
#include "OnlineError.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/ConfigCacheIni.h"

/** Json fields related to a friends list request */
#define FRIEND_JSON_FRIENDSLIST "data"
#define FRIEND_JSON_PAGING "paging"
#define FRIEND_JSON_NEXTURL "next"
#define FRIEND_JSON_SUMMARY "summary"
#define FRIEND_JSON_FRIENDCOUNT	"total_count"

// FOnlineFriendFacebook

FUniqueNetIdRef FOnlineFriendFacebook::GetUserId() const
{
	return UserIdPtr;
}

FString FOnlineFriendFacebook::GetRealName() const
{
	static FString NameField(TEXT(FRIEND_FIELD_NAME));
	FString Result;
	GetAccountData(NameField, Result);
	return Result;
}

FString FOnlineFriendFacebook::GetDisplayName(const FString& Platform) const
{
	static FString NameField(TEXT(FRIEND_FIELD_NAME));
	FString Result;
	GetAccountData(NameField, Result);
	return Result;
}

bool FOnlineFriendFacebook::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

EInviteStatus::Type FOnlineFriendFacebook::GetInviteStatus() const
{
	return EInviteStatus::Accepted;
}

const FOnlineUserPresence& FOnlineFriendFacebook::GetPresence() const
{
	return Presence;
}

bool FOnlineFriendFacebook::Parse(const TSharedPtr<FJsonObject>& JsonObject)
{
	bool bSuccess = false;

	if (FromJson(JsonObject))
	{
		if (!UserIdStr.IsEmpty())
		{
			UserIdPtr = FUniqueNetIdFacebook::Create(UserIdStr);
			bSuccess = true;
		}
	}

	return bSuccess;
}

// FOnlineFriendsFacebookCommon

FOnlineFriendsFacebookCommon::FOnlineFriendsFacebookCommon(FOnlineSubsystemFacebook* InSubsystem)
	: FacebookSubsystem(InSubsystem)
{
	check(FacebookSubsystem);

	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook.OnlineFriendsFacebook"), TEXT("FriendsUrl"), FriendsUrl, GEngineIni))
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Missing FriendsUrl= in [OnlineSubsystemFacebook.OnlineFriendsFacebook] of DefaultEngine.ini"));
	}

	FriendsUrl.ReplaceInline(TEXT("`ver"), *InSubsystem->GetAPIVer());

	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineFriendsFacebook"), TEXT("FriendsFields"), FriendsFields, GEngineIni);	

	// always required fields
	FriendsFields.AddUnique(TEXT(FRIEND_FIELD_ID));
	FriendsFields.AddUnique(TEXT(FRIEND_FIELD_NAME));
	FriendsFields.AddUnique(TEXT(FRIEND_FIELD_FIRSTNAME));
	FriendsFields.AddUnique(TEXT(FRIEND_FIELD_LASTNAME));
	FriendsFields.AddUnique(TEXT(FRIEND_FIELD_PICTURE));
}

FOnlineFriendsFacebookCommon::~FOnlineFriendsFacebookCommon()
{
}

bool FOnlineFriendsFacebookCommon::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate /*= FOnReadFriendsListComplete()*/)
{
	FString AccessToken;
	FString ErrorStr;

	if (!ListName.Equals(EFriendsLists::ToString(EFriendsLists::Default), ESearchCase::IgnoreCase))
	{
		// wrong list type
		ErrorStr = TEXT("Only the default friends list is supported");
	}
	else if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
	{
		// invalid local player index
		ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
	}
	else
	{
		// Make sure a registration request for this user is not currently pending
		for (TMap<IHttpRequest*, FPendingFriendsQuery>::TConstIterator It(FriendsQueryRequests); It; ++It)
		{
			const FPendingFriendsQuery& PendingFriendsQuery = It.Value();
			if (PendingFriendsQuery.LocalUserNum == LocalUserNum)
			{
				ErrorStr = FString::Printf(TEXT("Already pending friends read for LocalUserNum=%d."), LocalUserNum);
				break;
			}
		}

		AccessToken = FacebookSubsystem->GetIdentityInterface()->GetAuthToken(LocalUserNum);
		if (AccessToken.IsEmpty())
		{
			ErrorStr = FString::Printf(TEXT("Invalid access token for LocalUserNum=%d."), LocalUserNum);
		}
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("ReadFriendsList request failed. %s"), *ErrorStr);
		Delegate.ExecuteIfBound(LocalUserNum, false, ListName, ErrorStr);
		return false;
	}

	// Update cached entry for local user (done here because of pagination)
	FOnlineFriendsList& FriendsList = FriendsMap.FindOrAdd(LocalUserNum);
	FriendsList.Friends.Empty();
	
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	FriendsQueryRequests.Add(&HttpRequest.Get(), FPendingFriendsQuery(LocalUserNum));

	// Optional list of fields to query for each friend
	FString FieldsStr = FString::Join(FriendsFields, TEXT(","));

	// build the url
	FString FriendsQueryUrl = FriendsUrl.Replace(TEXT("`fields"), *FieldsStr, ESearchCase::IgnoreCase);
	FriendsQueryUrl = FriendsQueryUrl.Replace(TEXT("`token"), *AccessToken, ESearchCase::IgnoreCase);
	
	// kick off http request to read friends
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineFriendsFacebookCommon::QueryFriendsList_HttpRequestComplete, Delegate);
	HttpRequest->SetURL(FriendsQueryUrl);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetVerb(TEXT("GET"));
	return HttpRequest->ProcessRequest();
}

bool FOnlineFriendsFacebookCommon::DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate /*= FOnDeleteFriendsListComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, ListName, FString(TEXT("DeleteFriendsList() is not supported")));
	return false;
}

bool FOnlineFriendsFacebookCommon::SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate /*= FOnSendInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("SendInvite() is not supported")));
	return false;
}

bool FOnlineFriendsFacebookCommon::AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate /*= FOnAcceptInviteComplete()*/)
{
	Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, FString(TEXT("AcceptInvite() is not supported")));
	return false;
}

bool FOnlineFriendsFacebookCommon::RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnRejectInviteCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("RejectInvite() is not supported")));
	return false;
}

void FOnlineFriendsFacebookCommon::SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate /*= FOnSetFriendAliasComplete()*/)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	FacebookSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsFacebookCommon::SetFriendAlias is not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

void FOnlineFriendsFacebookCommon::DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	FacebookSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsFacebookCommon::DeleteFriendAlias is not supported"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

bool FOnlineFriendsFacebookCommon::DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TriggerOnDeleteFriendCompleteDelegates(LocalUserNum, false, FriendId, ListName, FString(TEXT("DeleteFriend() is not supported")));
	return false;
}

bool FOnlineFriendsFacebookCommon::GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends)
{
	bool bResult = false;
	if (ListName.Equals(EFriendsLists::ToString(EFriendsLists::Default), ESearchCase::IgnoreCase))
	{
		// valid local player index
		if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
		{
			// find friends list entry for local user
			const FOnlineFriendsList* FriendsList = FriendsMap.Find(LocalUserNum);
			if (FriendsList != NULL)
			{
				for (int32 FriendIdx = 0; FriendIdx < FriendsList->Friends.Num(); FriendIdx++)
				{
					OutFriends.Add(FriendsList->Friends[FriendIdx]);
				}
				bResult = true;
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Only the default friends list is supported"));
	}
	return bResult;
}

TSharedPtr<FOnlineFriend> FOnlineFriendsFacebookCommon::GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TSharedPtr<FOnlineFriend> Result = nullptr;
	if (ListName.Equals(EFriendsLists::ToString(EFriendsLists::Default), ESearchCase::IgnoreCase))
	{
		// valid local player index
		if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
		{
			// find friends list entry for local user
			const FOnlineFriendsList* FriendsList = FriendsMap.Find(LocalUserNum);
			if (FriendsList != NULL)
			{
				for (int32 FriendIdx = 0; FriendIdx < FriendsList->Friends.Num(); FriendIdx++)
				{
					if (*FriendsList->Friends[FriendIdx]->GetUserId() == FriendId)
					{
						Result = FriendsList->Friends[FriendIdx];
						break;
					}
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Only the default friends list is supported"));
	}

	return Result;
}

bool FOnlineFriendsFacebookCommon::IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	if (ListName.Equals(EFriendsLists::ToString(EFriendsLists::Default), ESearchCase::IgnoreCase))
	{
		TSharedPtr<FOnlineFriend> Friend = GetFriend(LocalUserNum, FriendId, ListName);
		if (Friend.IsValid() &&
			Friend->GetInviteStatus() == EInviteStatus::Accepted)
		{
			return true;
		}
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Only the default friends list is supported"));
	}

	return false;
}

bool FOnlineFriendsFacebookCommon::QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace)
{
	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsFacebookCommon::QueryRecentPlayers()"));

	TriggerOnQueryRecentPlayersCompleteDelegates(UserId, Namespace, false, TEXT("not implemented"));

	return false;
}

bool FOnlineFriendsFacebookCommon::GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers)
{
	return false;
}

void FOnlineFriendsFacebookCommon::DumpRecentPlayers() const
{

}

bool FOnlineFriendsFacebookCommon::BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsFacebookCommon::UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	return false;
}

bool FOnlineFriendsFacebookCommon::QueryBlockedPlayers(const FUniqueNetId& UserId)
{
	return false;
}

bool FOnlineFriendsFacebookCommon::GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers)
{
	return false;
}

void FOnlineFriendsFacebookCommon::QueryFriendsList_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FOnReadFriendsListComplete Delegate)
{
	bool bResult = false;
	bool bMoreToProcess = false;
	FString ResponseStr, ErrorStr;

	FPendingFriendsQuery PendingFriendsQuery = FriendsQueryRequests.FindRef(HttpRequest.Get());
	// Remove the request from list of pending entries
	FriendsQueryRequests.Remove(HttpRequest.Get());

	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOG_ONLINE_FRIEND(Verbose, TEXT("Query friends request complete. url=%s code=%d response=%s"),
				*HttpRequest->GetURL(), HttpResponse->GetResponseCode(), *ResponseStr);

			// Create the Json parser
			TSharedPtr<FJsonObject> JsonObject;
			TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(ResponseStr);

			if (FJsonSerializer::Deserialize(JsonReader, JsonObject) &&
				JsonObject.IsValid())
			{
				const TSharedPtr<FJsonObject>* PagingObject = nullptr;
				FString NextURL;
				if (JsonObject->TryGetObjectField(TEXT(FRIEND_JSON_PAGING), PagingObject))
				{
					// See if there are more entries to process after these
					(*PagingObject)->TryGetStringField(TEXT(FRIEND_JSON_NEXTURL), NextURL);
					bMoreToProcess = !NextURL.IsEmpty();
				}

				const TSharedPtr<FJsonObject>* JsonSummary = nullptr;
				if (JsonObject->TryGetObjectField(TEXT(FRIEND_JSON_SUMMARY), JsonSummary))
				{
					// This is not present when permissions aren't there
					int32 TotalCount = 0;
					(*JsonSummary)->TryGetNumberField(TEXT(FRIEND_JSON_FRIENDCOUNT), TotalCount);
					UE_LOG_ONLINE_FRIEND(Verbose, TEXT("Total friend count %d"), TotalCount);
				}

				FOnlineFriendsList& FriendsList = FriendsMap.FindOrAdd(PendingFriendsQuery.LocalUserNum);

				// Should have an array of id mappings
				TArray<TSharedPtr<FJsonValue> > JsonFriends = JsonObject->GetArrayField(TEXT(FRIEND_JSON_FRIENDSLIST));
				for (TArray<TSharedPtr<FJsonValue> >::TConstIterator FriendIt(JsonFriends); FriendIt; ++FriendIt)
				{
					TSharedPtr<FJsonObject> JsonFriendEntry = (*FriendIt)->AsObject();
					TSharedRef<FOnlineFriendFacebook> FriendEntry = MakeShared<FOnlineFriendFacebook>();
					if (FriendEntry->Parse(JsonFriendEntry))
					{
						// Add new friend entry to list
						FriendsList.Friends.Add(FriendEntry);
					}	
				}

				if (bMoreToProcess)
				{
					TSharedRef<IHttpRequest, ESPMode::ThreadSafe> NextHttpRequest = FHttpModule::Get().CreateRequest();
					FriendsQueryRequests.Add(&NextHttpRequest.Get(), FPendingFriendsQuery(PendingFriendsQuery.LocalUserNum));

					// read next page of friends
					NextHttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineFriendsFacebookCommon::QueryFriendsList_HttpRequestComplete, Delegate);
					NextHttpRequest->SetURL(NextURL);
					NextHttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
					NextHttpRequest->SetVerb(TEXT("GET"));
					NextHttpRequest->ProcessRequest();
				}

				bResult = true;
			}
		}
		else
		{
			ErrorStr = FString::Printf(TEXT("Invalid response. code=%d error=%s"),
				HttpResponse->GetResponseCode(), *ResponseStr);
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Query friends list request failed. %s"), *ErrorStr);
	}

	if (!bMoreToProcess)
	{
		Delegate.ExecuteIfBound(PendingFriendsQuery.LocalUserNum, bResult, EFriendsLists::ToString(EFriendsLists::Default), ErrorStr);
	}
}

void FOnlineFriendsFacebookCommon::DumpBlockedPlayers() const
{
}