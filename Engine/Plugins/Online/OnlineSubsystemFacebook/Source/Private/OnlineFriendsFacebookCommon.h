// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "OnlineSubsystemFacebookTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Http.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/** Details about a friend */
#define FRIEND_FIELD_ID "id"
#define FRIEND_FIELD_NAME "name"
#define FRIEND_FIELD_FIRSTNAME "first_name"
#define FRIEND_FIELD_LASTNAME "last_name"
#define FRIEND_FIELD_PICTURE "picture"

/**
 * Info associated with an online friend on the Facebook service
 */
class FOnlineFriendFacebook : 
	public FOnlineFriend,
	public FJsonSerializable
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineFriend
	
	virtual EInviteStatus::Type GetInviteStatus() const override;
	virtual const FOnlineUserPresence& GetPresence() const override;

	// FOnlineFriendFacebook

	/**
	 * Init/default constructor
	 */
	FOnlineFriendFacebook(const FString& InUserId=TEXT("")) 
		: UserIdPtr(FUniqueNetIdFacebook::Create(InUserId))
	{
	}

	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendFacebook()
	{
	}

	/**
	 * Parse Json friend request data
	 *
	 * @param JsonObject a json payload from a friend request
	 */
	bool Parse(const TSharedPtr<FJsonObject>& JsonObject);

private:

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** User Id */
	FString UserIdStr;
	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;
	/** Profile picture */
	FUserOnlineFacebookPicture Picture;
	/** Any addition account data associated with the friend */
	FJsonSerializableKeyValueMap AccountData;
	/** @temp presence info */
	FOnlineUserPresence Presence;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE(FRIEND_FIELD_ID, UserIdStr);
		JSON_SERIALIZE_OBJECT_SERIALIZABLE(FRIEND_FIELD_PICTURE, Picture);
		JSON_SERIALIZE_SIMPLECOPY(AccountData);
	END_JSON_SERIALIZER

	/** Allow the FB friends to fill in our private members from it's callbacks */
	friend class FOnlineFriendsFacebookCommon;
};

/**
 * Facebook service implementation of the online friends interface
 */
class FOnlineFriendsFacebookCommon :
	public IOnlineFriends
{	

public:

	// IOnlineFriends

	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName,  const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
 	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override;
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual void DumpRecentPlayers() const override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;

	// FOnlineFriendsFacebook

	/**
	 * Constructor
	 *
	 * @param InSubsystem Facebook subsystem being used
	 */
	FOnlineFriendsFacebookCommon(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendsFacebookCommon();

PACKAGE_SCOPE:
	/** For accessing identity/token info of user logged in */
	FOnlineSubsystemFacebook* FacebookSubsystem;
	/** List of online friends */
	struct FOnlineFriendsList
	{
		TArray< TSharedRef<FOnlineFriendFacebook> > Friends;
	};
	/** Cached friends list from last call to ReadFriendsList for each local user */
	TMap<int, FOnlineFriendsList> FriendsMap;

private:

	/**
	 * Should use the initialization constructor instead
	 */
	FOnlineFriendsFacebookCommon() = delete;

	/**
	 * Delegate called when a user /me request from Facebook is complete
	 */
	void QueryFriendsList_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FOnReadFriendsListComplete Delegate);

	/** Config based url for querying friends list */
	FString FriendsUrl;

	/** Config based list of fields to use when querying friends list */
	TArray<FString> FriendsFields;

	/** Info used to send request to register a user */
	struct FPendingFriendsQuery
	{
		FPendingFriendsQuery(int32 InLocalUserNum=0) 
			: LocalUserNum(InLocalUserNum)
		{
		}
		/** local index of user making the request */
		int32 LocalUserNum;
	};
	/** List of pending Http requests for user registration */
	TMap<class IHttpRequest*, FPendingFriendsQuery> FriendsQueryRequests;
};

typedef TSharedPtr<FOnlineFriendsFacebookCommon, ESPMode::ThreadSafe> FOnlineFriendsFacebookCommonPtr;
