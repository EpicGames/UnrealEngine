// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityFacebookCommon.h"
#if USES_RESTFUL_FACEBOOK
#include "OnlineIdentityFacebookRest.h"
#else // USES_RESTFUL_FACEBOOK
#include "OnlineIdentityFacebook.h"
#endif // USES_RESTFUL_FACEBOOK
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineSubsystemFacebookTypes.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineError.h"

FOnlineIdentityFacebookCommon::FOnlineIdentityFacebookCommon(FOnlineSubsystemFacebook* InSubsystem)
	: FacebookSubsystem(InSubsystem)
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("MeURL"), MeURL, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing MeURL= in [OnlineSubsystemFacebook.OnlineIdentityFacebook] of DefaultEngine.ini"));
	}

	MeURL.ReplaceInline(TEXT("`ver"), *InSubsystem->GetAPIVer());

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("ProfileFields"), ProfileFields, GEngineIni);
	// Add a few basic fields
	ProfileFields.AddUnique(TEXT(ME_FIELD_ID));
	ProfileFields.AddUnique(TEXT(ME_FIELD_NAME));
	ProfileFields.AddUnique(TEXT(ME_FIELD_FIRSTNAME));
	ProfileFields.AddUnique(TEXT(ME_FIELD_LASTNAME));
	ProfileFields.AddUnique(TEXT(ME_FIELD_PICTURE));
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityFacebookCommon::GetUserAccount(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> Result;

	const TSharedRef<FUserOnlineAccountFacebookCommon>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		Result = *FoundUserAccount;
	}

	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityFacebookCommon::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	
	for (FUserOnlineAccountFacebookMap::TConstIterator It(UserAccounts); It; ++It)
	{
		Result.Add(It.Value());
	}

	return Result;
}

FUniqueNetIdPtr FOnlineIdentityFacebookCommon::GetUniquePlayerId(int32 LocalUserNum) const
{
	const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);
	if (FoundId != nullptr)
	{
		return *FoundId;
	}
	return nullptr;
}

void FOnlineIdentityFacebookCommon::ProfileRequest(int32 LocalUserNum, const FString& AccessToken, const TArray<FString>& InProfileFields, FOnProfileRequestComplete& InCompletionDelegate)
{
	FString ErrorStr;
	bool bStarted = false;
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		if (!MeURL.IsEmpty())
		{
			if (!AccessToken.IsEmpty())
			{
				bStarted = true;

				// kick off http request to get user info with the access token
				TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
				LoginUserRequests.Add(&HttpRequest.Get(), FPendingLoginUser(LocalUserNum, AccessToken));

				FString FinalURL = MeURL.Replace(TEXT("`token"), *AccessToken, ESearchCase::IgnoreCase);
				if (InProfileFields.Num() > 0)
				{
					FinalURL += FString::Printf(TEXT("&fields=%s"), *FString::Join(InProfileFields, TEXT(",")));
				}

				HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineIdentityFacebookCommon::MeUser_HttpRequestComplete, InCompletionDelegate);
				HttpRequest->SetURL(FinalURL);
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetVerb(TEXT("GET"));
				HttpRequest->ProcessRequest();
			}
			else
			{
				ErrorStr = TEXT("No access token specified");
			}
		}
		else
		{
			ErrorStr = TEXT("No MeURL specified in DefaultEngine.ini");
		}
	}
	else
	{
		ErrorStr = TEXT("Invalid local user num");
	}

	if (!bStarted)
	{
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, ErrorStr);
	}
}

void FOnlineIdentityFacebookCommon::MeUser_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FOnProfileRequestComplete InCompletionDelegate)
{
	bool bResult = false;
	FString ResponseStr, ErrorStr;
	
	FPendingLoginUser PendingRegisterUser = LoginUserRequests.FindRef(HttpRequest.Get());
	// Remove the request from list of pending entries
	LoginUserRequests.Remove(HttpRequest.Get());

	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
#if UE_BUILD_SHIPPING
			static const FString URL = TEXT("[REDACTED]");
#else
			const FString URL = HttpRequest->GetURL();
#endif
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("RegisterUser request complete. url=%s code=%d response=%s"),
				*URL, HttpResponse->GetResponseCode(), *ResponseStr);

			TSharedRef<FUserOnlineAccountFacebook> User = MakeShared<FUserOnlineAccountFacebook>();
			if (User->Parse(PendingRegisterUser.AccessToken, ResponseStr))
			{
				// update/add cached entry for user
				UserAccounts.Add(User->GetUserId()->ToString(), User);
				// keep track of user ids for local users
				UserIds.Add(PendingRegisterUser.LocalUserNum, User->GetUserId());

				bResult = true;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Error parsing login. payload=%s"),
					*ResponseStr);
			}
		}
		else
		{
			FErrorFacebook Error;
			Error.FromJson(ResponseStr);
			if (Error.Error.Type == TEXT("OAuthException"))
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("OAuthError: %s"), *Error.ToDebugString());
				ErrorStr = FB_AUTH_EXPIRED_CREDS;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Invalid response. code=%d error=%s"),
					HttpResponse->GetResponseCode(), *ResponseStr);
			}
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("RegisterUser request failed. %s"), *ErrorStr);
	}

	InCompletionDelegate.ExecuteIfBound(PendingRegisterUser.LocalUserNum, bResult, ErrorStr);
}

void FOnlineIdentityFacebookCommon::RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& InCompletionDelegate)
{
	IOnlineSharingPtr SharingInt = FacebookSubsystem->GetSharingInterface();
	if (ensure(SharingInt.IsValid()))
	{
		SharingInt->RequestCurrentPermissions(LocalUserNum, InCompletionDelegate);
	}
	else
	{
		FString ErrorStr = TEXT("No sharing interface, unable to request current sharing permissions");
		TArray<FSharingPermission> EmptyPermissions;
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, EmptyPermissions);
	}
}

FUniqueNetIdPtr FOnlineIdentityFacebookCommon::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes != nullptr && Size > 0)
	{
		FString StrId = FString::ConstructFromPtrSize((TCHAR*)Bytes, Size);
		return FUniqueNetIdFacebook::Create(StrId);
	}
	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityFacebookCommon::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdFacebook::Create(Str);
}

bool FOnlineIdentityFacebookCommon::AutoLogin(int32 LocalUserNum)
{
	return false;
}

ELoginStatus::Type FOnlineIdentityFacebookCommon::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityFacebookCommon::GetLoginStatus(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(UserId);
	if (UserAccount.IsValid() &&
		UserAccount->GetUserId()->IsValid() &&
		!UserAccount->GetAccessToken().IsEmpty())
	{
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FString FOnlineIdentityFacebookCommon::GetPlayerNickname(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetPlayerNickname(*UserId);
	}
	return TEXT("");
}

FString FOnlineIdentityFacebookCommon::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	const TSharedRef<FUserOnlineAccountFacebookCommon>* FoundUserAccount = UserAccounts.Find(UserId.ToString());
	if (FoundUserAccount != nullptr)
	{
		const TSharedRef<FUserOnlineAccountFacebookCommon>& UserAccount = *FoundUserAccount;
		return UserAccount->GetRealName();
	}
	return TEXT("");
}

FString FOnlineIdentityFacebookCommon::GetAuthToken(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			return UserAccount->GetAccessToken();
		}
	}
	return FString();
}

void FOnlineIdentityFacebookCommon::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityFacebookCommon::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	FacebookSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentityFacebookCommon::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}	

FPlatformUserId FOnlineIdentityFacebookCommon::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityFacebookCommon::GetAuthType() const
{
	return AUTH_TYPE_FACEBOOK;
}

