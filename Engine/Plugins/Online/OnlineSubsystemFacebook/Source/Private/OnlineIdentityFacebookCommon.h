// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineSharingInterface.h"
#include "OnlineSubsystemFacebookPackage.h"
#include "Interfaces/IHttpRequest.h"

class FOnlineSubsystemFacebook;
class FUserOnlineAccountFacebookCommon;

#define AUTH_TYPE_FACEBOOK TEXT("facebook")
#define FB_AUTH_EXPIRED_CREDS TEXT("com.epicgames.fb.oauth.expiredcreds");

/**
 * Delegate fired after a Facebook profile request has been completed
 *
 * @param LocalUserNum the controller number of the associated user
 * @param bWasSuccessful was the request successful
 * @param ErrorStr error associated with the request
 */
DECLARE_DELEGATE_ThreeParams(FOnProfileRequestComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*ErrorStr*/);

/** Mapping from user id to internal online account info (only one per user) */
typedef TMap<FString, TSharedRef<FUserOnlineAccountFacebookCommon> > FUserOnlineAccountFacebookMap;

/**
 * Facebook service implementation of the online identity interface
 */
class FOnlineIdentityFacebookCommon :
	public IOnlineIdentity
{

protected:

	/** Parent subsystem */
	FOnlineSubsystemFacebook* FacebookSubsystem;
	/** URL for Facebook API to retrieve personal details */
	FString MeURL;
	/** Users that have been registered/authenticated */
	FUserOnlineAccountFacebookMap UserAccounts;
	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr > UserIds;

public:

	// IOnlineIdentity
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;

	// FOnlineIdentityFacebook

	FOnlineIdentityFacebookCommon(FOnlineSubsystemFacebook* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityFacebookCommon()
	{
	}

PACKAGE_SCOPE:

	/**
	 * Retrieve the profile for a given user and access token
	 *
	 * @param LocalUserNum the controller number of the associated user
	 * @param AccessToken associated access token to make the request
	 * @param InProfileFields profile fields to retrieve
	 * @param InCompletionDelegate delegate to fire when request is completed
	 */
	void ProfileRequest(int32 LocalUserNum, const FString& AccessToken, const TArray<FString>& InProfileFields, FOnProfileRequestComplete& InCompletionDelegate);

	/**
	 * Retrieve the permissions for a given user and access token
	 * 
	 * @param LocalUserNum the controller number of the associated user
	 * @param InCompletionDelegate delegate to fire when request is completed
	 */
	void RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& InCompletionDelegate);

protected:

	/** Profile fields */
	TArray<FString> ProfileFields;

private:

	/**
	 * Delegate called when a user /me request from Facebook is complete
	 */
	void MeUser_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FOnProfileRequestComplete InCompletionDelegate);

	/** Info used to send request to register a user */
	struct FPendingLoginUser
	{
		FPendingLoginUser(
			int32 InLocalUserNum=0, 
			const FString& InAccessToken=FString()
			) 
			: LocalUserNum(InLocalUserNum)
			, AccessToken(InAccessToken)
		{

		}
		/** local index of user being registered */
		int32 LocalUserNum;
		/** Access token being used to login to Facebook */
		FString AccessToken;
	};
	/** List of pending Http requests for user registration */
	TMap<IHttpRequest*, FPendingLoginUser> LoginUserRequests;
};

typedef TSharedPtr<FOnlineIdentityFacebookCommon, ESPMode::ThreadSafe> FOnlineIdentityFacebookCommonPtr;
