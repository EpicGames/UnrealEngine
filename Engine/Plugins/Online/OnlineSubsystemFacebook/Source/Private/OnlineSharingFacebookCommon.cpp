// Copyright Epic Games, Inc. All Rights Reserved.

// Module includes
#include "OnlineSharingFacebookCommon.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebookCommon.h"
#include "OnlineSubsystemFacebookTypes.h"
#include "Misc/ConfigCacheIni.h"

void FFacebookPermissions::GetPermissions(TArray<FSharingPermission>& OutPermissions)
{
	OutPermissions.Empty(GrantedPerms.Num() + DeclinedPerms.Num());
	OutPermissions.Append(GrantedPerms);
	OutPermissions.Append(DeclinedPerms);
}

void FFacebookPermissions::Setup()
{
	///////////////////////////////////////////////
	// Read Permissions
	SharingPermissionsMap.Add(EOnlineSharingCategory::Friends, { TEXT(PERM_READ_FRIENDS) });
	SharingPermissionsMap.Add(EOnlineSharingCategory::Email, { TEXT(PERM_READ_EMAIL) });
	SharingPermissionsMap.Add(EOnlineSharingCategory::ProfileInfo, { TEXT(PERM_PUBLIC_PROFILE), TEXT(PERM_PUBLIC_GAMING_PROFILE) });
}

void FFacebookPermissions::Reset()
{
	GrantedPerms.Empty();
	DeclinedPerms.Empty();
}

EOnlineSharingCategory FFacebookPermissions::GetCategoryFromFacebookPermission(const FString& FacebookPermission) const
{
	EOnlineSharingCategory Cat = EOnlineSharingCategory::None;
	for (const auto& Iter : SharingPermissionsMap)
	{
		EOnlineSharingCategory TmpCat = Iter.Key;
		const TArray<FString>& TmpValue = Iter.Value;
		if (TmpValue.Contains(FacebookPermission))
		{
			Cat = TmpCat;
			break;
		}
	}
	return Cat;
}

bool FFacebookPermissions::RefreshPermissions(const FString& NewJsonStr)
{
	if (!NewJsonStr.IsEmpty())
	{
		FFacebookPermissionsJson Data;
		if (Data.FromJson(NewJsonStr))
		{
			GrantedPerms.Empty(Data.Permissions.Num());
			DeclinedPerms.Empty(Data.Permissions.Num());

			const FString Granted(TEXT(PERM_GRANTED));
			const FString Declined(TEXT(PERM_DECLINED));
			for (const FFacebookPermissionsJson::FFacebookPermissionJson& Perm : Data.Permissions)
			{
				const EOnlineSharingCategory Cat = GetCategoryFromFacebookPermission(Perm.Name);

				if (Cat != EOnlineSharingCategory::None)
				{
					FSharingPermission NewPerm(Perm.Name, Cat);
					if (Perm.Status == Granted)
					{
						NewPerm.Status = EOnlineSharingPermissionState::Granted;
						GrantedPerms.Add(MoveTemp(NewPerm));
					}
					else if (Perm.Status == Declined)
					{
						NewPerm.Status = EOnlineSharingPermissionState::Declined;
						DeclinedPerms.Add(MoveTemp(NewPerm));
					}
					else
					{
						UE_LOG_ONLINE_SHARING(Warning, TEXT("Unknown permission status %s %s"), *Perm.Name, *Perm.Status);
					}
				}
				else
				{
					UE_LOG_ONLINE_SHARING(Warning, TEXT("Permission not mapped to any category %s"), *Perm.Name);
				}
			}
			return true;
		}
	}

	return false;
}

void FFacebookPermissions::RefreshPermissions(const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
{
	GrantedPerms.Empty(GrantedPermissions.Num());
	DeclinedPerms.Empty(DeclinedPermissions.Num());

	for (const FString& PermissionName : GrantedPermissions)
	{
		const EOnlineSharingCategory Cat = GetCategoryFromFacebookPermission(PermissionName);

		if (Cat != EOnlineSharingCategory::None)
		{
			FSharingPermission NewPerm(PermissionName, Cat);
			GrantedPerms.Add(MoveTemp(NewPerm));
		}
		else
		{
			UE_LOG_ONLINE_SHARING(Warning, TEXT("Permission not mapped to any category %s"), *PermissionName);
		}
	}

	for (const FString& PermissionName : DeclinedPermissions)
	{
		const EOnlineSharingCategory Cat = GetCategoryFromFacebookPermission(PermissionName);

		if (Cat != EOnlineSharingCategory::None)
		{
			FSharingPermission NewPerm(PermissionName, Cat);
			DeclinedPerms.Add(MoveTemp(NewPerm));
		}
		else
		{
			UE_LOG_ONLINE_SHARING(Warning, TEXT("Permission not mapped to any category %s"), *PermissionName);
		}
	}
}

bool FFacebookPermissions::HasPermission(EOnlineSharingCategory RequestedPermission, TArray<FSharingPermission>& OutMissingPermissions) const
{
	for (FSharingPermissionsMap::TConstIterator It(SharingPermissionsMap); It; ++It)
	{
		EOnlineSharingCategory Category = It.Key();
		if ((RequestedPermission & Category) != EOnlineSharingCategory::None)
		{
			const TArray<FString>& PermsReqd = It.Value();
			UE_LOG_ONLINE_SHARING(Display, TEXT("PermissionsMap[%s] - [%i]"), ToString(Category), PermsReqd.Num());
			for (const FString& PermReqd : PermsReqd)
			{
				if (!GrantedPerms.ContainsByPredicate([PermReqd](FSharingPermission& PermToCheck) { return PermToCheck.Name == PermReqd; }))
				{
					FSharingPermission Permission(PermReqd, Category);
					if (DeclinedPerms.ContainsByPredicate([PermReqd](FSharingPermission& PermToCheck) { return PermToCheck.Name == PermReqd; }))
					{
						Permission.Status = EOnlineSharingPermissionState::Declined;
					}

					OutMissingPermissions.AddUnique(Permission);
				}
			}
		}
	}

	return (OutMissingPermissions.Num() == 0);
}

FOnlineSharingFacebookCommon::FOnlineSharingFacebookCommon(FOnlineSubsystemFacebook* InSubsystem)
	: Subsystem(InSubsystem)
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook.OnlineSharingFacebook"), TEXT("PermissionsURL"), PermissionsURL, GEngineIni))
	{
		UE_LOG_ONLINE_SHARING(Warning, TEXT("Missing PermissionsURL= in [OnlineSubsystemFacebook.OnlineSharingFacebook] of DefaultEngine.ini"));
	}

	PermissionsURL.ReplaceInline(TEXT("`ver"), *InSubsystem->GetAPIVer());

	CurrentPermissions.Setup();

	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	check(IdentityInt.IsValid());
	for (int32 i = 0; i < MAX_LOCAL_PLAYERS; i++)
	{
		LoginStatusChangedDelegates[i] = IdentityInt->AddOnLoginStatusChangedDelegate_Handle(i, FOnLoginStatusChangedDelegate::CreateRaw(this, &FOnlineSharingFacebookCommon::OnLoginStatusChanged));
	}
}

FOnlineSharingFacebookCommon::~FOnlineSharingFacebookCommon()
{
	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		for (int32 i = 0; i < MAX_LOCAL_PLAYERS; i++)
		{
			IdentityInt->ClearOnLoginStatusChangedDelegate_Handle(i, LoginStatusChangedDelegates[i]);
		}
	}
}

void FOnlineSharingFacebookCommon::SetCurrentPermissions(const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions)
{
	CurrentPermissions.RefreshPermissions(GrantedPermissions, DeclinedPermissions);
}

void FOnlineSharingFacebookCommon::OnLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& UserId)
{
	if (OldStatus == ELoginStatus::LoggedIn && NewStatus == ELoginStatus::NotLoggedIn)
	{
		CurrentPermissions.Reset();
	}
}

void FOnlineSharingFacebookCommon::RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& CompletionDelegate)
{
	FString Error;
	bool bStarted = false;
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		if (!PermissionsURL.IsEmpty())
		{
			const FString AccessToken = Subsystem->GetIdentityInterface()->GetAuthToken(LocalUserNum);
			if (!AccessToken.IsEmpty())
			{
				bStarted = true;

				// kick off http request to get user info with the access token
				FString FinalURL = PermissionsURL.Replace(TEXT("`token"), *AccessToken, ESearchCase::IgnoreCase);

				TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
				HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOnlineSharingFacebookCommon::Permissions_HttpComplete, LocalUserNum, CompletionDelegate);
				HttpRequest->SetURL(FinalURL);
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetVerb(TEXT("GET"));
				HttpRequest->ProcessRequest();
			}
			else
			{
				Error = TEXT("No access token specified");
			}
		}
		else
		{
			Error = TEXT("No MeURL specified in DefaultEngine.ini");
		}
	}
	else
	{
		Error = TEXT("Invalid local user num");
	}

	if (!bStarted)
	{
		TArray<FSharingPermission> Permissions;
		CurrentPermissions.GetPermissions(Permissions);
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false, Permissions);
	}
}

void FOnlineSharingFacebookCommon::Permissions_HttpComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 LocalUserNum, FOnRequestCurrentPermissionsComplete CompletionDelegate)
{
	bool bResult = false;
	FString ResponseStr, ErrorStr;

	if (bSucceeded &&
		HttpResponse.IsValid())
	{
		ResponseStr = HttpResponse->GetContentAsString();
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
#if UE_BUILD_SHIPPING
			static const FString URL = TEXT("[REDACTED]");
#else
			const FString URL = HttpRequest->GetURL();
#endif
			UE_LOG_ONLINE_SHARING(Verbose, TEXT("Permissions request complete. url=%s code=%d response=%s"),
				*URL, HttpResponse->GetResponseCode(), *ResponseStr);

			if (CurrentPermissions.RefreshPermissions(ResponseStr))
			{
				bResult = true;
			}
			else
			{
				UE_LOG_ONLINE_SHARING(Verbose, TEXT("Failed to parse permissions"));
			}
		}
		else
		{
			FErrorFacebook Error;
			Error.FromJson(ResponseStr);
			if (Error.Error.Type == TEXT("OAuthException"))
			{
				UE_LOG_ONLINE_SHARING(Warning, TEXT("OAuthError: %s"), *Error.ToDebugString());
				ErrorStr = FB_AUTH_EXPIRED_CREDS;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Invalid response. code=%d error=%s"),
					HttpResponse->GetResponseCode(), *ResponseStr);
			}
		}
	}
	else
	{
		ErrorStr = TEXT("No response");
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_SHARING(Warning, TEXT("Permissions request failed. %s"), *ErrorStr);
	}

	TArray<FSharingPermission> Permissions;
	GetCurrentPermissions(LocalUserNum, Permissions);
	CompletionDelegate.ExecuteIfBound(LocalUserNum, bResult, Permissions);
}

void FOnlineSharingFacebookCommon::GetCurrentPermissions(int32 LocalUserNum, TArray<FSharingPermission>& OutPermissions)
{
	return CurrentPermissions.GetPermissions(OutPermissions);
}

bool FOnlineSharingFacebookCommon::RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions)
{
	/** NYI */
	ensure((NewPermissions & ~EOnlineSharingCategory::ReadPermissionMask) == EOnlineSharingCategory::None);
	bool bTriggeredRequest = false;
	TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

bool FOnlineSharingFacebookCommon::RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy)
{
	/** NYI */
	ensure((NewPermissions & ~EOnlineSharingCategory::PublishPermissionMask) == EOnlineSharingCategory::None);
	bool bTriggeredRequest = false;
	TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

bool FOnlineSharingFacebookCommon::ShareStatusUpdate(int32 LocalUserNum, const FOnlineStatusUpdate& StatusUpdate)
{
	/** NYI */
	bool bTriggeredRequest = false;
	TriggerOnSharePostCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

bool FOnlineSharingFacebookCommon::ReadNewsFeed(int32 LocalUserNum, int32 NumPostsToRead)
{
	/** NYI */
	bool bTriggeredRequest = false;
	TriggerOnReadNewsFeedCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

EOnlineCachedResult::Type FOnlineSharingFacebookCommon::GetCachedNewsFeed(int32 LocalUserNum, int32 NewsFeedIdx, FOnlineStatusUpdate& OutNewsFeed)
{
	check(NewsFeedIdx >= 0);
	UE_LOG_ONLINE_SHARING(Error, TEXT("FOnlineSharingFacebookCommon::GetCachedNewsFeed not yet implemented"));
	return EOnlineCachedResult::NotFound;
}

EOnlineCachedResult::Type FOnlineSharingFacebookCommon::GetCachedNewsFeeds(int32 LocalUserNum, TArray<FOnlineStatusUpdate>& OutNewsFeeds)
{
	UE_LOG_ONLINE_SHARING(Error, TEXT("FOnlineSharingFacebookCommon::GetCachedNewsFeeds not yet implemented"));
	return EOnlineCachedResult::NotFound;
}
