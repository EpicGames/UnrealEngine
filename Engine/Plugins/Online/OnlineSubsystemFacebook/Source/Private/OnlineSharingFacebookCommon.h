// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
// Module includes
#include "CoreMinimal.h"
#include "Interfaces/OnlineSharingInterface.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/IHttpRequest.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/** Permissions to request at login */
#define PERM_PUBLIC_PROFILE "public_profile"
#define PERM_PUBLIC_GAMING_PROFILE "gaming_profile" // Same as public_profile for facebook apps with Gaming Services type
#define PERM_READ_EMAIL "email"

/** Elevated read permissions */
#define PERM_READ_FRIENDS "user_friends"

/** Facebook Permission Json */
#define PERM_JSON_PERMISSIONS "data"
#define PERM_JSON_PERMISSION_NAME "permission"
#define PERM_JSON_PERMISSION_STATUS "status"

/** Facebook Permission status */
#define PERM_GRANTED "granted"
#define PERM_DECLINED "declined"

/**
 * Registry of known permissions associated with a logged in user
 */
class FFacebookPermissions
{

private:

	// The read permissions map which sets up the Facebook permissions in their correct category
	typedef TMap<EOnlineSharingCategory, TArray<FString>> FSharingPermissionsMap;
	FSharingPermissionsMap SharingPermissionsMap;

	class FFacebookPermissionsJson : public FJsonSerializable
	{
		public:
			class FFacebookPermissionJson :
				public FJsonSerializable
			{

			public:
				/** Name of the permission */
				FString Name;
				/** Status of the permission (granted/declined) */
				FString Status;

				FFacebookPermissionJson() {}

				// FJsonSerializable
				BEGIN_JSON_SERIALIZER
					JSON_SERIALIZE(PERM_JSON_PERMISSION_NAME, Name);
					JSON_SERIALIZE(PERM_JSON_PERMISSION_STATUS, Status);
				END_JSON_SERIALIZER
			};


			/** Main error body */
			TArray<FFacebookPermissionJson> Permissions;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE_ARRAY_SERIALIZABLE(PERM_JSON_PERMISSIONS, Permissions, FFacebookPermissionJson);
			END_JSON_SERIALIZER
	};

	EOnlineSharingCategory GetCategoryFromFacebookPermission(const FString& FacebookPermission) const;

	/** List of known permissions to have been accepted by the user */
	TArray<FSharingPermission> GrantedPerms;
	/** List of known permissions intentionally declined by the user */
	TArray<FSharingPermission> DeclinedPerms;

public:

	FFacebookPermissions() {}
	~FFacebookPermissions() {}

	/**
	 * Get the current list of permissions, granted and denied
	 *
	 * @param OutPermission store the current permissions
	 */
	void GetPermissions(TArray<FSharingPermission>& OutPermissions);

	/**
	 * Setup the permission categories with the relevant Facebook entries
	 */
	void Setup();

	/**
	 * Clear out all permissions
	 */
	void Reset();

	/**
	 * Reset the current permissions, filling them in from a json string
	 *
	 * @param NewJsonStr valid json response from Facebook permissions request
	 */
	bool RefreshPermissions(const FString& NewJsonStr);

	/**
	 * Reset the current permissions, filling them from string arrays
	 *
	 * @param GrantedPermissions array of accepted Facebook permissions
	 * @param DeclinedPermissions array of declined Facebook permissions
	 */
	void RefreshPermissions(const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions);

	/**
	 * Has this user granted the proper permissions for a given category
	 * 
	 * @param RequestedPermission type of permission to check for
	 * @param OutMissingPermissions if the function returns false, this is the list of permissions still needed to proceed
	 *
	 * @return true if the user has granted all permissions related to the category, false if there are missing permissions
	 */
	bool HasPermission(EOnlineSharingCategory RequestedPermission, TArray<FSharingPermission>& OutMissingPermissions) const;
};


/**
 * Facebook implementation of the Online Sharing Interface
 */
class FOnlineSharingFacebookCommon : public IOnlineSharing
{

public:

	//~ Begin IOnlineSharing Interface
	virtual void RequestCurrentPermissions(int32 LocalUserNum, FOnRequestCurrentPermissionsComplete& CompletionDelegate) override;
	virtual void GetCurrentPermissions(int32 LocalUserNum, TArray<FSharingPermission>& OutPermissions) override;
	virtual bool ReadNewsFeed(int32 LocalUserNum, int32 NumPostsToRead) override;
	virtual bool RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions) override;
	virtual bool ShareStatusUpdate(int32 LocalUserNum, const FOnlineStatusUpdate& StatusUpdate) override;
	virtual bool RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy) override;
	virtual EOnlineCachedResult::Type GetCachedNewsFeed(int32 LocalUserNum, int32 NewsFeedIdx, FOnlineStatusUpdate& OutNewsFeed) override;
	virtual EOnlineCachedResult::Type GetCachedNewsFeeds(int32 LocalUserNum, TArray<FOnlineStatusUpdate>& OutNewsFeeds) override;
	//~ End IOnlineSharing Interface

public:

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineSharingFacebookCommon(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineSharingFacebookCommon();

	/**
	 * Reset the current permissions, filling them from string arrays
	 *
	 * @param GrantedPermissions array of accepted Facebook permissions
	 * @param DeclinedPermissions array of declined Facebook permissions
	 */
	void SetCurrentPermissions(const TArray<FString>& GrantedPermissions, const TArray<FString>& DeclinedPermissions);

protected:

	/** Parent subsystem */
	FOnlineSubsystemFacebook* Subsystem;
	/** Permissions request URL */
	FString PermissionsURL;
	/** Current state of granted/declined permissions */
	FFacebookPermissions CurrentPermissions;

private:

	/**
	 * Delegate fired when the current permissions request completes
	 *
	 * @param HttpRequest the permission request
	 * @param HttpResposne the permission response
	 * @param bSucceeded did the request succeed
	 * @param LocalUserNum local user that made the request
	 * @param CompletionDelegate follow up delegate to fire now that this request is complete
	 */
	void Permissions_HttpComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 LocalUserNum, FOnRequestCurrentPermissionsComplete CompletionDelegate);

	/**
	 * Delegate fired when login status changes
	 * need to reset permissions on logout
	 *
	 * @param LocalUserNum local user of interest
	 * @param OldStatus previous login status
	 * @param NewStatus current login status
	 * @param UserId user id of interest
	 */
	void OnLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& UserId);
	FDelegateHandle LoginStatusChangedDelegates[MAX_LOCAL_PLAYERS];
};


typedef TSharedPtr<FOnlineSharingFacebookCommon, ESPMode::ThreadSafe> FOnlineSharingFacebookCommonPtr;
