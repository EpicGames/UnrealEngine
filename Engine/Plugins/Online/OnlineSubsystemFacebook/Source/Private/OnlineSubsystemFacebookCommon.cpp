// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemFacebookCommon.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebookCommon.h"
#include "OnlineFriendsFacebookCommon.h"
#include "OnlineSharingFacebookCommon.h"
#include "OnlineUserFacebookCommon.h"
#include "OnlineExternalUIFacebookCommon.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"

/** Fallback to latest tested API version */
#define FACEBOOK_API_VER TEXT("v2.12")

FOnlineSubsystemFacebookCommon::FOnlineSubsystemFacebookCommon(FName InInstanceName)
	: FOnlineSubsystemImpl(FACEBOOK_SUBSYSTEM, InInstanceName)
	, FacebookIdentity(nullptr)
	, FacebookFriends(nullptr)
	, FacebookSharing(nullptr)
	, FacebookUser(nullptr)
	, FacebookExternalUI(nullptr)
{
}

FOnlineSubsystemFacebookCommon::~FOnlineSubsystemFacebookCommon()
{
}

bool FOnlineSubsystemFacebookCommon::Init()
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook"), TEXT("ClientId"), ClientId, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("Missing ClientId= in [OnlineSubsystemFacebook] of DefaultEngine.ini"));
	}

	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook"), TEXT("APIVer"), APIVer, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("Missing APIVer= in [OnlineSubsystemFacebook] of DefaultEngine.ini"));
		APIVer = FACEBOOK_API_VER;
	}

	return true;
}

bool FOnlineSubsystemFacebookCommon::Shutdown()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemFacebookCommon::Shutdown()"));

	FOnlineSubsystemImpl::Shutdown();

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	DESTRUCT_INTERFACE(FacebookSharing);
	DESTRUCT_INTERFACE(FacebookExternalUI);
	DESTRUCT_INTERFACE(FacebookFriends);
	DESTRUCT_INTERFACE(FacebookUser);
	DESTRUCT_INTERFACE(FacebookIdentity);

#undef DESTRUCT_INTERFACE

	return true;
}

bool FOnlineSubsystemFacebookCommon::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemFacebookCommon_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	return true;
}

FString FOnlineSubsystemFacebookCommon::GetAppId() const
{
	return ClientId;
}

bool FOnlineSubsystemFacebookCommon::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("FACEBOOK")))
	{
		bWasHandled = HandleFacebookExecCommands(InWorld, Cmd, Ar);
	}
	else if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		bWasHandled = true;
	}
	return bWasHandled;
}

IOnlineSessionPtr FOnlineSubsystemFacebookCommon::GetSessionInterface() const
{
	return nullptr;
}

IOnlineFriendsPtr FOnlineSubsystemFacebookCommon::GetFriendsInterface() const
{
	return FacebookFriends;
}

IOnlinePartyPtr FOnlineSubsystemFacebookCommon::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemFacebookCommon::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemFacebookCommon::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemFacebookCommon::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemFacebookCommon::GetLeaderboardsInterface() const
{
	return nullptr;
}

IOnlineVoicePtr FOnlineSubsystemFacebookCommon::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemFacebookCommon::GetExternalUIInterface() const	
{
	return FacebookExternalUI;
}

IOnlineTimePtr FOnlineSubsystemFacebookCommon::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemFacebookCommon::GetIdentityInterface() const
{
	return FacebookIdentity;
}

IOnlineTitleFilePtr FOnlineSubsystemFacebookCommon::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemFacebookCommon::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineEventsPtr FOnlineSubsystemFacebookCommon::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemFacebookCommon::GetAchievementsInterface() const
{
	return nullptr;
}

IOnlineSharingPtr FOnlineSubsystemFacebookCommon::GetSharingInterface() const
{
	return FacebookSharing;
}

IOnlineUserPtr FOnlineSubsystemFacebookCommon::GetUserInterface() const
{
	return FacebookUser;
}

IOnlineMessagePtr FOnlineSubsystemFacebookCommon::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemFacebookCommon::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemFacebookCommon::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemFacebookCommon::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemFacebookCommon::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemFacebookCommon::GetTournamentInterface() const
{
	return nullptr;
}

FText FOnlineSubsystemFacebookCommon::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemFacebook", "OnlineServiceName", "Facebook");
}

static bool ParseFacebookCommandArgsLocalNum(const TCHAR* Cmd, int32& LocalNum)
{
	FString LocalNumStr = FParse::Token(Cmd, false);
	LocalNum = FCString::Atoi(*LocalNumStr);
	return (!LocalNumStr.IsEmpty() && LocalNum >= 0 && LocalNum <= MAX_LOCAL_PLAYERS);
}

static bool ParseFacebookCommandArgsScopesAndLocalNum(const TCHAR* Cmd, EOnlineSharingCategory Mask, EOnlineSharingCategory& Scope, int32& LocalNum)
{
	FString ScopeStr = FParse::Token(Cmd, false);
	Scope = EOnlineSharingCategory(FParse::HexNumber(*ScopeStr));
	if ((Scope & Mask) == EOnlineSharingCategory::None)
	{
		return false;
	}

	return ParseFacebookCommandArgsLocalNum(Cmd, LocalNum);
}

bool FOnlineSubsystemFacebookCommon::HandleFacebookExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

	if (FParse::Command(&Cmd, TEXT("LOGIN")))
	{
		bWasHandled = true;

		int32 LocalNum = 0;
		if (!ParseFacebookCommandArgsLocalNum(Cmd, LocalNum))
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: LOGIN <localnum>"));
		}
		else
		{
			HandleFacebookLoginCommand(LocalNum);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("LOGOUT")))
	{
		bWasHandled = true;

		int32 LocalNum = 0;
		if (!ParseFacebookCommandArgsLocalNum(Cmd, LocalNum))
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: LOGOUT <localnum>"));
		}
		else
		{
			HandleFacebookLogoutCommand(LocalNum);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("FRIENDS")))
	{
		bWasHandled = true;

		int32 LocalNum = 0;
		if (!ParseFacebookCommandArgsLocalNum(Cmd, LocalNum))
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: LOGOUT <localnum>"));
		}
		else
		{
			HandleFacebookFriendsCommand(LocalNum);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("REQUESTREADSCOPES")))
	{
		bWasHandled = true;

		EOnlineSharingCategory Scope = EOnlineSharingCategory::None;
		int32 LocalNum = 0;

		if (!ParseFacebookCommandArgsScopesAndLocalNum(Cmd, EOnlineSharingCategory::ReadPermissionMask, Scope, LocalNum))
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: REQUESTREADSCOPES <read categories> <localnum>"));
		}
		else
		{
			HandleFacebookRequestReadScopesCommand(Scope, LocalNum);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("REQUESTPUBLISHSCOPES")))
	{
		bWasHandled = true;

		EOnlineSharingCategory Scope = EOnlineSharingCategory::None;
		int32 LocalNum = 0;

		if (!ParseFacebookCommandArgsScopesAndLocalNum(Cmd, EOnlineSharingCategory::PublishPermissionMask, Scope, LocalNum))
		{
			UE_LOG_ONLINE(Warning, TEXT("usage: REQUESTPUBLISHSCOPES <publish categories> <localnum>"));
		}
		else
		{
			HandleFacebookRequestPublishScopesCommand(Scope, LocalNum);
		}
	}
	return bWasHandled;
}

void FOnlineSubsystemFacebookCommon::HandleFacebookLoginCommand(int32 LocalNum)
{
	IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		if (IdentityInt->GetLoginStatus(LocalNum) == ELoginStatus::LoggedIn)
		{
			UE_LOG_ONLINE(Warning, TEXT("User %d already logged into Facebook. Nickname %s"), LocalNum, *IdentityInt->GetPlayerNickname(LocalNum));
		}
		else
		{
			FOnlineAccountCredentials AccountCredentials("", "", IdentityInt->GetAuthType());

			TSharedPtr<FDelegateHandle> LoginCompleteHandle = MakeShared<FDelegateHandle>();
			*LoginCompleteHandle = IdentityInt->AddOnLoginCompleteDelegate_Handle(LocalNum, FOnLoginCompleteDelegate::CreateLambda([IdentityInt = FacebookIdentity.Get(), LoginCompleteHandle](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& /*UserId*/, const FString& Error)
				{
					if (bWasSuccessful)
					{
						UE_LOG_ONLINE(Log, TEXT("Facebook login succeeded for player %d. User nickname %s"), LocalUserNum, *IdentityInt->GetPlayerNickname(LocalUserNum));
					}
					else
					{
						UE_LOG_ONLINE(Log, TEXT("Facebook login failed for player %d: Error: %s"), LocalUserNum, *Error);
					}
					IdentityInt->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *LoginCompleteHandle);
				}));
			IdentityInt->Login(LocalNum, AccountCredentials);
		}
	}
}

void FOnlineSubsystemFacebookCommon::HandleFacebookLogoutCommand(int32 LocalNum)
{
	IOnlineIdentityPtr IdentityInt = GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		if (IdentityInt->GetLoginStatus(LocalNum) == ELoginStatus::NotLoggedIn)
		{
			UE_LOG_ONLINE(Warning, TEXT("User %d not logged into Facebook"), LocalNum);
			return;
		}
		TSharedPtr<FDelegateHandle> LogoutCompleteHandle = MakeShared<FDelegateHandle>();
		*LogoutCompleteHandle = IdentityInt->AddOnLogoutCompleteDelegate_Handle(LocalNum, FOnLogoutCompleteDelegate::CreateLambda([IdentityInt = IdentityInt.Get(), LogoutCompleteHandle](int32 LocalUserNum, bool bWasSuccessful)
			{
				UE_LOG_ONLINE(Log, TEXT("Facebook logout %s for player %d"), bWasSuccessful ? TEXT("succeeded") : TEXT("failed"), LocalUserNum);
				IdentityInt->ClearOnLogoutCompleteDelegate_Handle(LocalUserNum, *LogoutCompleteHandle);
			}));
		IdentityInt->Logout(LocalNum);
	}
}

void FOnlineSubsystemFacebookCommon::HandleFacebookFriendsCommand(int32 LocalNum)
{
	IOnlineFriendsPtr FriendsInt = GetFriendsInterface();
	if (FriendsInt.IsValid())
	{
		FriendsInt->ReadFriendsList(LocalNum, ToString(EFriendsLists::Default), FOnReadFriendsListComplete::CreateLambda([FriendsInt = FriendsInt.Get()](int32 LocalUserNum, bool bWasSuccessful, const FString& ListName, const FString& ErrorStr) {
			if (bWasSuccessful)
			{
				UE_LOG_ONLINE(Log, TEXT("Retrieve Facebook friends list %s for player %d succeeded"), *ListName, LocalUserNum);
				TArray<TSharedRef<FOnlineFriend>> Friends;
				FriendsInt->GetFriendsList(LocalUserNum, ListName, Friends);
				UE_LOG_ONLINE(Log, TEXT("List of %d friends: %s"), Friends.Num(), *FString::JoinBy(Friends, TEXT(", "), [](const TSharedRef<FOnlineFriend>& Friend) { return Friend->GetRealName(); }));
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Retrieve Facebook friends list %s for player %d failed: Error: %s"), *ListName, LocalUserNum, *ErrorStr);
			}
			}));
	}
}

void FOnlineSubsystemFacebookCommon::HandleFacebookRequestPublishScopesCommand(EOnlineSharingCategory Scopes, int32 LocalNum)
{
	IOnlineSharingPtr SharingInt = GetSharingInterface();
	if (SharingInt.IsValid())
	{
		TSharedPtr<FDelegateHandle> Handle = MakeShared<FDelegateHandle>();
		*Handle = SharingInt->AddOnRequestNewPublishPermissionsCompleteDelegate_Handle(LocalNum, FOnRequestNewPublishPermissionsCompleteDelegate::CreateLambda([Handle, Scopes](int32 LocalUserNum, bool bWasSuccessful) mutable
			{
				UE_LOG_ONLINE(Log, TEXT("Publish permission request for player %d %s"), LocalUserNum, bWasSuccessful?TEXT("succeeded"):TEXT("failed"));
				Handle.Reset();
			}));
		SharingInt->RequestNewPublishPermissions(LocalNum, Scopes, EOnlineStatusUpdatePrivacy::Everyone);
	}
}

void FOnlineSubsystemFacebookCommon::HandleFacebookRequestReadScopesCommand(EOnlineSharingCategory Scopes, int32 LocalNum)
{
	IOnlineSharingPtr SharingInt = GetSharingInterface();
	if (SharingInt.IsValid())
	{
		TSharedPtr<FDelegateHandle> Handle = MakeShared<FDelegateHandle>();
		*Handle = SharingInt->AddOnRequestNewReadPermissionsCompleteDelegate_Handle(LocalNum, FOnRequestNewReadPermissionsCompleteDelegate::CreateLambda([Handle, Scopes](int32 LocalUserNum, bool bWasSuccessful) mutable
			{
				UE_LOG_ONLINE(Log, TEXT("Read permission request for player %d %s"), LocalUserNum, bWasSuccessful?TEXT("succeeded"):TEXT("failed"));
				Handle.Reset();
			}));
		SharingInt->RequestNewReadPermissions(LocalNum, Scopes);
	}
}
