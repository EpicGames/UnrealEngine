// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemFacebookPackage.h"
#include "Serialization/JsonSerializerMacros.h"

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityFacebookCommon, ESPMode::ThreadSafe> FOnlineIdentityFacebookCommonPtr;
typedef TSharedPtr<class FOnlineFriendsFacebookCommon, ESPMode::ThreadSafe> FOnlineFriendsFacebookCommonPtr;
typedef TSharedPtr<class FOnlineSharingFacebookCommon, ESPMode::ThreadSafe> FOnlineSharingFacebookCommonPtr;
typedef TSharedPtr<class FOnlineUserFacebookCommon, ESPMode::ThreadSafe> FOnlineUserFacebookCommonPtr;
typedef TSharedPtr<class FOnlineExternalUIFacebookCommon, ESPMode::ThreadSafe> FOnlineExternalUIFacebookCommonPtr;

/**
 *	OnlineSubsystemFacebookCommon - Implementation of the online subsystem for Facebook services
 */
class ONLINESUBSYSTEMFACEBOOK_API FOnlineSubsystemFacebookCommon 
	: public FOnlineSubsystemImpl
{
private:

	/**
	 * Exec function handling for Exec() call
	 */
	bool HandleFacebookExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	/** Handle facebook login console command */
	void HandleFacebookLoginCommand(int32 LocalNum);
	/** Handle facebook logout console command */
	void HandleFacebookLogoutCommand(int32 LocalNum);
	/** Handle facebook friends console command */
	void HandleFacebookFriendsCommand(int32 LocalNum);
	/** Handle facebook request additional scopes console command */
	void HandleFacebookRequestReadScopesCommand(EOnlineSharingCategory Scopes, int32 LocalNum);
	void HandleFacebookRequestPublishScopesCommand(EOnlineSharingCategory Scopes, int32 LocalNum);

public:

	// IOnlineSubsystem Interface
	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override { return nullptr; }
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override { return nullptr; }
	virtual IOnlineEventsPtr GetEventsInterface() const override;
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlineSharingPtr GetSharingInterface() const override;
	virtual IOnlineUserPtr GetUserInterface() const override;
	virtual IOnlineMessagePtr GetMessageInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override;
	virtual IOnlineChatPtr GetChatInterface() const override;
	virtual IOnlineStatsPtr GetStatsInterface() const override;
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual FText GetOnlineServiceName() const override;

	// FTickerBaseObject

	virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemFacebookCommon

	/** @return the API version in use (depends on the Facebook app settings) */
	const FString& GetAPIVer() const { return APIVer; }

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemFacebookCommon();

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemFacebookCommon() = delete;
	explicit FOnlineSubsystemFacebookCommon(FName InInstanceName);

protected:

	/** The client id given to us by Facebook */
	FString ClientId;
	/** Version of the API in use (depends on the Facebook app settings) */
	FString APIVer;

	/** Facebook implementation of identity interface */
	FOnlineIdentityFacebookCommonPtr FacebookIdentity;

	/** Facebook implementation of friends interface */
	FOnlineFriendsFacebookCommonPtr FacebookFriends;

	/** Facebook implementation of sharing interface */
	FOnlineSharingFacebookCommonPtr FacebookSharing;

	/** Facebook implementation of user interface */
	FOnlineUserFacebookCommonPtr FacebookUser;

	/** Facebook implementation of the external ui */
	FOnlineExternalUIFacebookCommonPtr FacebookExternalUI;
};

typedef TSharedPtr<FOnlineSubsystemFacebookCommon, ESPMode::ThreadSafe> FOnlineSubsystemFacebookCommonPtr;
