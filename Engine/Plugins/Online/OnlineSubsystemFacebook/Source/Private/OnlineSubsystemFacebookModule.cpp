// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemFacebookModule.h"
#include "OnlineSubsystemFacebookPrivate.h"

IMPLEMENT_MODULE(FOnlineSubsystemFacebookModule, OnlineSubsystemFacebook);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryFacebook : public IOnlineFactory
{
public:

	FOnlineFactoryFacebook() {}
	virtual ~FOnlineFactoryFacebook() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemFacebookPtr OnlineSub = MakeShared<FOnlineSubsystemFacebook, ESPMode::ThreadSafe>(InstanceName);
		if (OnlineSub->IsEnabled())
		{
			UE_LOG_ONLINE(Log, TEXT("Facebook API is being initialized."));

			if(!OnlineSub->Init())
			{
				UE_LOG_ONLINE(Warning, TEXT("Facebook API failed to initialize!"));
				OnlineSub->Shutdown();
				OnlineSub.Reset();
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Facebook API disabled!"));
			OnlineSub->Shutdown();
			OnlineSub.Reset();
		}

		return OnlineSub;
	}
};

void FOnlineSubsystemFacebookModule::StartupModule()
{
	UE_LOG_ONLINE(Log, TEXT("Facebook Module Startup!"));

	FacebookFactory = new FOnlineFactoryFacebook();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(FACEBOOK_SUBSYSTEM, FacebookFactory);
}

void FOnlineSubsystemFacebookModule::ShutdownModule()
{
	UE_LOG_ONLINE(Log, TEXT("Facebook Module Shutdown!"));

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(FACEBOOK_SUBSYSTEM);

	delete FacebookFactory;
	FacebookFactory = NULL;
}