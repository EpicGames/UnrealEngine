// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "Http.h"

#include "OnlineSubsystemFacebook.h"
#include "OnlineSubsystemFacebookModule.h"

/** pre-pended to all Facebook logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("Facebook: ")
