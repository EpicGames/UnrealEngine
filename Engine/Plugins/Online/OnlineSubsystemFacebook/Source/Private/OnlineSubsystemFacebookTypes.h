// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineSubsystemFacebookPackage.h"

#define PICTURE_DATA "data"
#define PICTURE_DATA_SILHOUETTE "is_silhouette"
#define PICTURE_DATA_URL "url"

using FUniqueNetIdFacebookPtr = TSharedPtr<const class FUniqueNetIdFacebook>;
using FUniqueNetIdFacebookRef = TSharedRef<const class FUniqueNetIdFacebook>;

/**
 * Facebook specific implementation of the unique net id
 */
class FUniqueNetIdFacebook :
	public FUniqueNetId
{
PACKAGE_SCOPE:
	/** Holds the net id for a player */
	uint64 UniqueNetId;

public:
	template<typename... TArgs>
	static FUniqueNetIdFacebookRef Create(TArgs&&... Args)
	{
		return MakeShareable(new FUniqueNetIdFacebook(Forward<TArgs>(Args)...));
	}

	//~ Begin FUniqueNetId Interface
	virtual FName GetType() const override
	{
		return FACEBOOK_SUBSYSTEM;
	}

	virtual const uint8* GetBytes() const override
	{
		return (uint8*)&UniqueNetId;
	}


	virtual int32 GetSize() const override
	{
		return sizeof(uint64);
	}

	virtual bool IsValid() const override
	{
		return UniqueNetId != 0;
	}


	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("%I64d"), UniqueNetId);
	}


	virtual FString ToDebugString() const override
	{
		const FString UniqueNetIdStr = FString::Printf(TEXT("0%I64X"), UniqueNetId);
		return OSS_UNIQUEID_REDACT(*this, UniqueNetIdStr);
	}

	virtual uint32 GetTypeHash() const override
	{
		return ::GetTypeHash(UniqueNetId);
	}
	//~ End FUniqueNetId Interface

	/** global static instance of invalid (zero) id */
	static const FUniqueNetIdRef& EmptyId()
	{
		static const FUniqueNetIdRef EmptyId(Create());
		return EmptyId;
	}

private:
	/** Hidden on purpose */
	FUniqueNetIdFacebook() :
		UniqueNetId(0)
	{
	}

	/**
	 * Copy Constructor
	 *
	 * @param Src the id to copy
	 */
	explicit FUniqueNetIdFacebook(const FUniqueNetIdFacebook& Src) :
		UniqueNetId(Src.UniqueNetId)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdFacebook(uint64 InUniqueNetId) :
		UniqueNetId(InUniqueNetId)
	{
	}

	explicit FUniqueNetIdFacebook(const FString& Str) :
		UniqueNetId(FCString::Strtoui64(*Str, nullptr, 10))
	{
	}
};

/**
 * Facebook error from JSON payload
 */
class FErrorFacebook :
	public FJsonSerializable
{
public:

	/**
	 * Constructor
	 */
	FErrorFacebook()
	{
	}

	class FErrorBody :
		public FJsonSerializable
	{

	public:
		/** Facebook error message */
		FString Message;
		/** Type of error reported by Facebook  */
		FString Type;
		/** Facebook error code */
		int32 Code;
		/** Facebook error sub code */
		int32 ErrorSubCode;
		/** Facebook trace id */
		FString FBTraceId;

		FErrorBody() {}

		// FJsonSerializable
		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("message", Message);
			JSON_SERIALIZE("type", Type);
			JSON_SERIALIZE("code", Code);
			JSON_SERIALIZE("error_subcode", ErrorSubCode);
			JSON_SERIALIZE("fbtrace_id", FBTraceId);
		END_JSON_SERIALIZER
	};

	/** Main error body */
	FErrorBody Error;

	/** @return debug output for logs */
	FString ToDebugString() const { return FString::Printf(TEXT("%s [Type:%s Code:%d SubCode:%d Trace:%s]"), *Error.Message, *Error.Type, Error.Code, Error.ErrorSubCode, *Error.FBTraceId); }

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_OBJECT_SERIALIZABLE("error", Error);
	END_JSON_SERIALIZER
};

/** Facebook profile picture */
class FUserOnlineFacebookPicture :
	public FJsonSerializable
{
public:

	struct FPictureData :
		public FJsonSerializable
	{
		/** Is this picture the default silhouette */
		bool bIsSilhouette;
		/** URL to picture content */
		FString PictureURL;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE(PICTURE_DATA_SILHOUETTE, bIsSilhouette);
			JSON_SERIALIZE(PICTURE_DATA_URL, PictureURL);
		END_JSON_SERIALIZER
	};

	/** User picture */
	FPictureData PictureData;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_OBJECT_SERIALIZABLE(PICTURE_DATA, PictureData);
	END_JSON_SERIALIZER
};
