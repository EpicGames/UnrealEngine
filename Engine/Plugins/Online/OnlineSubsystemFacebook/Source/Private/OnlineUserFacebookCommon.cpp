// Copyright Epic Games, Inc. All Rights Reserved.


// Module includes
#include "OnlineUserFacebookCommon.h"
#include "OnlineSubsystemFacebookPrivate.h"

// FOnlineUserInfoFacebook

FUniqueNetIdRef FOnlineUserInfoFacebook::GetUserId() const
{
	return UserId;
}

FString FOnlineUserInfoFacebook::GetRealName() const
{
	FString Result;
	GetAccountData(TEXT("name"), Result);
	return Result;
}

FString FOnlineUserInfoFacebook::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(TEXT("username"), Result);
	return Result;
}

bool FOnlineUserInfoFacebook::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

// FOnlineUserFacebookCommon

FOnlineUserFacebookCommon::FOnlineUserFacebookCommon(FOnlineSubsystemFacebookCommon* InSubsystem)
	: Subsystem(InSubsystem)
{
}

FOnlineUserFacebookCommon::~FOnlineUserFacebookCommon()
{
}

bool FOnlineUserFacebookCommon::QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef >& UserIds)
{
	bool bTriggeredRequest = false;
	TriggerOnQueryUserInfoCompleteDelegates(LocalUserNum, false, UserIds, TEXT("Not implemented"));
	return bTriggeredRequest;
}

bool FOnlineUserFacebookCommon::GetAllUserInfo(int32 LocalUserNum, TArray< TSharedRef<FOnlineUser> >& OutUsers)
{
	UE_LOG_ONLINE_USER(Verbose, TEXT("FOnlineUserFacebookCommon::GetAllUserInfo()"));
	for (int32 Idx=0; Idx < CachedUsers.Num(); Idx++)
	{
		OutUsers.Add(CachedUsers[Idx]);
	}

	return true;
}

TSharedPtr<FOnlineUser> FOnlineUserFacebookCommon::GetUserInfo(int32 LocalUserNum, const FUniqueNetId& UserId)
{
	TSharedPtr<FOnlineUser> Result;

	UE_LOG_ONLINE_USER(Verbose, TEXT("FOnlineUserFacebookCommon::GetUserInfo()"));

	for (int32 Idx=0; Idx < CachedUsers.Num(); Idx++)
	{
		if (*(CachedUsers[Idx]->GetUserId()) == UserId)
		{
			Result = CachedUsers[Idx];
			break;
		}
	}
	return Result;
}

bool FOnlineUserFacebookCommon::QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate)
{
	Delegate.ExecuteIfBound(false, UserId, DisplayNameOrEmail, *FUniqueNetIdFacebook::EmptyId(), TEXT("not implemented"));
	return false;
}

bool FOnlineUserFacebookCommon::QueryExternalIdMappings(const FUniqueNetId& LocalUserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate)
{
	Delegate.ExecuteIfBound(false, LocalUserId, QueryOptions, ExternalIds, TEXT("not implemented"));
	return false;
}

void FOnlineUserFacebookCommon::GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds)
{
	// Not implemented for Facebook - return an array full of empty values
	OutIds.SetNum(ExternalIds.Num());
}

FUniqueNetIdPtr FOnlineUserFacebookCommon::GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId)
{
	return FUniqueNetIdPtr();
}
