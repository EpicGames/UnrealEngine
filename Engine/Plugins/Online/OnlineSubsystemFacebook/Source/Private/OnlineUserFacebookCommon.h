// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
 
// Module includes
#include "Interfaces/OnlineUserInterface.h"
#include "OnlineSubsystemFacebookPackage.h"
#include "OnlineSubsystemFacebookTypes.h"

class FOnlineSubsystemFacebookCommon;

/**
 * Info associated with an online user on the facebook service
 */
class FOnlineUserInfoFacebook : 
	public FOnlineUser
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineUserInfoFacebook

	/**
	 * Init/default constructor
	 */
	FOnlineUserInfoFacebook(const FString& InUserId=TEXT("")) 
		: UserId(FUniqueNetIdFacebook::Create(InUserId))
	{
	}

	/**
	 * Destructor
	 */
	virtual ~FOnlineUserInfoFacebook()
	{
	}

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserId;
	/** Any addition account data associated with the friend */
	TMap<FString, FString> AccountData;
};


/**
 * Facebook implementation of the Online User Interface
 */
class FOnlineUserFacebookCommon : public IOnlineUser
{

public:
	
	// IOnlineUser

	virtual bool QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef >& UserIds) override;
	virtual bool GetAllUserInfo(int32 LocalUserNum, TArray< TSharedRef<class FOnlineUser> >& OutUsers) override;
	virtual TSharedPtr<FOnlineUser> GetUserInfo(int32 LocalUserNum, const class FUniqueNetId& UserId) override;	
	virtual bool QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate = FOnQueryUserMappingComplete()) override;
	virtual bool QueryExternalIdMappings(const FUniqueNetId& LocalUserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate = FOnQueryExternalIdMappingsComplete()) override;
	virtual void GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds) override;
	virtual FUniqueNetIdPtr GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId) override;

	// FOnlineUserFacebookCommon

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineUserFacebookCommon(FOnlineSubsystemFacebookCommon* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineUserFacebookCommon();


protected:

	FOnlineSubsystemFacebookCommon* Subsystem;

	/** The collection of Facebook users received through the callbacks in QueryUserInfo */
	TArray< TSharedRef<FOnlineUserInfoFacebook> > CachedUsers;
};

typedef TSharedPtr<FOnlineUserFacebookCommon, ESPMode::ThreadSafe> FOnlineUserFacebookCommonPtr;
