// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineExternalUIInterfaceFacebookRest.h"
#include "OnlineSubsystemFacebook.h"
#include "OnlineIdentityFacebookRest.h"
#include "PlatformHttp.h"
#include "OnlineError.h"

#define FB_STATE_TOKEN TEXT("state")
#define FB_ACCESS_TOKEN TEXT("access_token")
#define FB_ERRORCODE_TOKEN TEXT("error_code")
#define FB_ERRORDESC_TOKEN TEXT("error_description")

bool FOnlineExternalUIFacebook::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	bool bStarted = false;
	FString ErrorStr;
	if (ControllerIndex >= 0 && ControllerIndex < MAX_LOCAL_PLAYERS)
	{
		FOnlineIdentityFacebookPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(FacebookSubsystem->GetIdentityInterface());
		if (IdentityInt.IsValid())
		{
			const FFacebookLoginURL& URLDetails = IdentityInt->GetLoginURLDetails();
			if (URLDetails.IsValid())
			{
				const FString RequestedURL = URLDetails.GetURL();
				bool bShouldContinueLoginFlow = false;
				FOnLoginRedirectURL OnRedirectURLDelegate = FOnLoginRedirectURL::CreateRaw(this, &FOnlineExternalUIFacebook::OnLoginRedirectURL);
				FOnLoginFlowComplete OnExternalLoginFlowCompleteDelegate = FOnLoginFlowComplete::CreateRaw(this, &FOnlineExternalUIFacebook::OnExternalLoginFlowComplete, ControllerIndex, Delegate);
				TriggerOnLoginFlowUIRequiredDelegates(RequestedURL, OnRedirectURLDelegate, OnExternalLoginFlowCompleteDelegate, bShouldContinueLoginFlow);
				bStarted = bShouldContinueLoginFlow;
			}
			else
			{
				ErrorStr = TEXT("ShowLoginUI: Url Details not properly configured");
			}
		}
		else
		{
			ErrorStr = TEXT("ShowLoginUI: Missing identity interface");
		}
	}
	else
	{
		ErrorStr = FString::Printf(TEXT("ShowLoginUI: Invalid controller index (%d)"), ControllerIndex);
	}

	if (!bStarted)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("%s"), *ErrorStr);

		FOnlineError Error;
		Error.SetFromErrorCode(MoveTemp(ErrorStr));
		FacebookSubsystem->ExecuteNextTick([ControllerIndex, Delegate, Error = MoveTemp(Error)]()
		{
			Delegate.ExecuteIfBound(nullptr, ControllerIndex, Error);
		});
	}

	return bStarted;
}

FLoginFlowResult FOnlineExternalUIFacebook::ParseRedirectResult(const FFacebookLoginURL& URLDetails, const FString& RedirectURL)
{
	FLoginFlowResult Result;

	TMap<FString, FString> ParamsMap;
	{
		FStringView QueryParams;
		FStringView UriFragment;

		int32 QueryParamsStart = RedirectURL.Find(TEXT("?"));
		int32 FragmentStart = RedirectURL.Find(TEXT("#"), ESearchCase::IgnoreCase, ESearchDir::FromStart, QueryParamsStart);
		
		bool HasQueryParams = QueryParamsStart != INDEX_NONE;
		bool HasUriFragment = FragmentStart != INDEX_NONE;

		if (HasQueryParams)
		{
			// Discard "?"
			QueryParamsStart += 1;
			int32 QueryParamsLength = HasUriFragment ? (FragmentStart - QueryParamsStart) : (RedirectURL.Len() - 1);
			QueryParams = FStringView(RedirectURL).Mid(QueryParamsStart, QueryParamsLength);
			HasQueryParams = QueryParams.IsEmpty();
		}

		if (HasUriFragment)
		{
			UriFragment = FStringView(RedirectURL).RightChop(FragmentStart);

			// Facebook adds a fragment #_=_ when that field is left blank
			// https://developers.facebook.com/blog/post/552/
			if (UriFragment.Equals(TEXT("#_=_")))
			{
				UriFragment.Reset();
				HasUriFragment = false;
			}
			else
			{
				UriFragment.RemovePrefix(1);
			}
		}

		// Auth token response is received through the UriFragment
		// Error notifications are received through the QueryParams
		TArray<FString> Params;
		if (HasUriFragment)
		{
			FString(UriFragment).ParseIntoArray(Params, TEXT("&"));
		}
		else
		{
			FString(QueryParams).ParseIntoArray(Params, TEXT("&"));
		}

		for (FString& Param : Params)
		{
			FString Key, Value;
			if (Param.Split(TEXT("="), &Key, &Value))
			{
				ParamsMap.Add(Key, Value);
			}
		}
	}

	// Both error or success should contain the state token
	// Otherwise the response does not refer to this request
	const FString* State = ParamsMap.Find(FB_STATE_TOKEN);
	if (State)
	{
		if (URLDetails.State == *State)
		{
			const FString* AccessToken = ParamsMap.Find(FB_ACCESS_TOKEN);
			if (AccessToken)
			{
				Result.Error.bSucceeded = true;
				Result.Token = *AccessToken;
			}
			else
			{
				const FString* ErrorCode = ParamsMap.Find(FB_ERRORCODE_TOKEN);
				if (ErrorCode)
				{
					Result.Error.ErrorRaw = RedirectURL;

					const FString* ErrorDesc = ParamsMap.Find(FB_ERRORDESC_TOKEN);
					if (ErrorDesc)
					{
						Result.Error.ErrorMessage = FText::FromString(*ErrorDesc);
					}

					Result.Error.ErrorCode = *ErrorCode;
					Result.NumericErrorCode = FPlatformString::Atoi(**ErrorCode);
				}
				else
				{
					// Set some default in case parsing fails
					Result.Error.ErrorRaw = LOGIN_ERROR_UNKNOWN;
					Result.Error.ErrorMessage = FText::FromString(LOGIN_ERROR_UNKNOWN);
					Result.Error.ErrorCode = LOGIN_ERROR_UNKNOWN;
					Result.NumericErrorCode = -1;
				}
			}
		}
	}

	return Result;
}

FLoginFlowResult FOnlineExternalUIFacebook::OnLoginRedirectURL(const FString& RedirectURL)
{
	FLoginFlowResult Result;

	FOnlineIdentityFacebookPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(FacebookSubsystem->GetIdentityInterface());
	if (IdentityInt.IsValid())
	{  
		const FFacebookLoginURL& URLDetails = IdentityInt->GetLoginURLDetails();
		if (URLDetails.IsValid())
		{
			// Wait for the RedirectURI to appear
			if (!RedirectURL.Contains(FPlatformHttp::UrlEncode(URLDetails.LoginUrl)))
			{
				if (RedirectURL.StartsWith(URLDetails.LoginRedirectUrl))
				{
					Result = ParseRedirectResult(URLDetails, RedirectURL);
				}
				else
				{
					static const FString FacebookHelpURL = TEXT("https://www.facebook.com/login/help.php");
					if (RedirectURL.StartsWith(FacebookHelpURL))
					{
						Result.Error.ErrorRaw = LOGIN_ERROR_AUTH_FAILURE;
						Result.Error.ErrorMessage = FText::FromString(LOGIN_ERROR_AUTH_FAILURE);
						Result.Error.ErrorCode = LOGIN_ERROR_AUTH_FAILURE;
						Result.NumericErrorCode = -2;
					}
				}
			}
		}
	}

	return Result;
}

void FOnlineExternalUIFacebook::OnExternalLoginFlowComplete(const FLoginFlowResult& Result, int ControllerIndex, const FOnLoginUIClosedDelegate Delegate)
{
	UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("OnExternalLoginFlowComplete %s"), *Result.ToDebugString());

	bool bStarted = false;
	if (Result.IsValid())
	{
		FOnlineIdentityFacebookPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityFacebook>(FacebookSubsystem->GetIdentityInterface());
		if (IdentityInt.IsValid())
		{
			bStarted = true;

			FOnLoginCompleteDelegate CompletionDelegate;
			CompletionDelegate = FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineExternalUIFacebook::OnAccessTokenLoginComplete, Delegate);
			IdentityInt->Login(ControllerIndex, Result.Token, CompletionDelegate);
		}
	}

	if (!bStarted)
	{
		FOnlineError LoginFlowError = Result.Error;
		FacebookSubsystem->ExecuteNextTick([ControllerIndex, LoginFlowError, Delegate]()
		{
			Delegate.ExecuteIfBound(nullptr, ControllerIndex, LoginFlowError);
		});
	}
}

void FOnlineExternalUIFacebook::OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate)
{
	FUniqueNetIdPtr StrongUserId = UserId.AsShared();
	FacebookSubsystem->ExecuteNextTick([StrongUserId, LocalUserNum, bWasSuccessful, Delegate]()
	{
		Delegate.ExecuteIfBound(StrongUserId, LocalUserNum, FOnlineError(bWasSuccessful));
	});
}

#endif // USES_RESTFUL_FACEBOOK