// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_FACEBOOK

#include "OnlineSubsystemFacebook.h"
#include "OnlineExternalUIFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;
class IHttpRequest;
struct FFacebookLoginURL;

/** 
 * Implementation for the Facebook external UIs
 */
class FOnlineExternalUIFacebook : public FOnlineExternalUIFacebookCommon
{
private:

PACKAGE_SCOPE:

	/** 
	 * Constructor
	 * @param InSubsystem The owner of this external UI interface.
	 */
	FOnlineExternalUIFacebook(FOnlineSubsystemFacebook* InSubsystem)
		: FOnlineExternalUIFacebookCommon(InSubsystem)
	{
	}

public:

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUIFacebook()
	{
	}

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;

private: 

	/**
	 * Parse a successful URL redirect from Facebook
	 *
	 * @param URLDetails URL config settings
	 * @param RedirectURL an URL knowns to start with the redirect URI
	 */
	FLoginFlowResult ParseRedirectResult(const FFacebookLoginURL& URLDetails, const FString& RedirectURL);

	/**
	 * Delegate fired when redirect URLs from the login flow are passed back for parsing
	 * We are looking for the success or error completion state from Facebook to grab the access token or complete the flow
	 *
	 * @param RedirectURL URL received from the login flow for parsing
	 */
	FLoginFlowResult OnLoginRedirectURL(const FString& RedirectURL);

	/**
	 * Delegate fired when the login flow is complete
	 *
	 * @param Result final result of the login flow action
	 * @param ControllerIndex index of the local user initiating the request
	 * @param Delegate UI closed delegate to fire, completing the external UIs part in the login process
	 */
	void OnExternalLoginFlowComplete(const FLoginFlowResult& Result, int ControllerIndex, const FOnLoginUIClosedDelegate Delegate);

	/**
	 * Delegate fired when the Facebook identity interface has completed login using the token retrieved from the login flow
	 *
	 * @param LocalUserNum index of the local user initiating the request
	 * @param bWasSuccessful was the login call successful
	 * @param UserId user id of the logged in user, or null if login failed
	 * @param Error error string if applicable
	 * @param Delegate UI closed delegate to fire, completing the external UIs part in the login process
	 */
	void OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate);
};

typedef TSharedPtr<FOnlineExternalUIFacebook, ESPMode::ThreadSafe> FOnlineExternalUIFacebookPtr;

#endif // USES_RESTFUL_FACEBOOK