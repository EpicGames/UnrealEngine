// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineFriendsFacebookRest.h"
#include "OnlineSubsystemFacebookPrivate.h"

FOnlineFriendsFacebook::FOnlineFriendsFacebook(FOnlineSubsystemFacebook* InSubsystem) 
	: FOnlineFriendsFacebookCommon(InSubsystem)
{
}

FOnlineFriendsFacebook::~FOnlineFriendsFacebook()
{
}

#endif // USES_RESTFUL_FACEBOOK
