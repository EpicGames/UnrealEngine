// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_FACEBOOK

#include "OnlineFriendsFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/**
 * Facebook service implementation of the online friends interface (Windows)
 */
class FOnlineFriendsFacebook :
	public FOnlineFriendsFacebookCommon
{	

public:

	/**
	 * Constructor
	 *
	 * @param InSubsystem Facebook subsystem being used
	 */
	FOnlineFriendsFacebook(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendsFacebook();

private:

	/**
	 * Should use the initialization constructor instead
	 */
	FOnlineFriendsFacebook() = delete;

};

typedef TSharedPtr<FOnlineFriendsFacebook, ESPMode::ThreadSafe> FOnlineFriendsFacebookPtr;

#endif // USES_RESTFUL_FACEBOOK
