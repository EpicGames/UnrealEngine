// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineIdentityFacebookRest.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineExternalUIInterfaceFacebookRest.h"
#include "Misc/ConfigCacheIni.h"

FOnlineIdentityFacebook::FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineIdentityFacebookCommon(InSubsystem)
	, bHasLoginOutstanding(false)
{
	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("LoginUrl"), LoginURLDetails.LoginUrl, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing LoginUrl= in [OnlineSubsystemFacebook.OnlineIdentityFacebook] of DefaultEngine.ini"));
	}
	if (!GConfig->GetString(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("LoginRedirectUrl"), LoginURLDetails.LoginRedirectUrl, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing LoginRedirectUrl= in [OnlineSubsystemFacebook.OnlineIdentityFacebook] of DefaultEngine.ini"));
	}
	if (!GConfig->GetBool(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("bUsePopup"), LoginURLDetails.bUsePopup, GEngineIni))
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing bUsePopup= in [OnlineSubsystemFacebook.OnlineIdentityFacebook] of DefaultEngine.ini"));
	}

	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("LoginDomains"), LoginDomains, GEngineIni);

	LoginURLDetails.ClientId = InSubsystem->GetAppId();
	if (LoginURLDetails.ClientId.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Missing ClientId= in [OnlineSubsystemFacebook] of DefaultEngine.ini"));
	}

	LoginURLDetails.LoginUrl.ReplaceInline(TEXT("`ver"), *InSubsystem->GetAPIVer());

	// Setup permission scope fields
	GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("ScopeFields"), LoginURLDetails.ScopeFields, GEngineIni);
	// always required login access fields
	LoginURLDetails.ScopeFields.AddUnique(TEXT(PERM_PUBLIC_PROFILE));
}

bool FOnlineIdentityFacebook::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	FString ErrorStr;

	if (bHasLoginOutstanding)
	{
		ErrorStr = FString::Printf(TEXT("Registration already pending for user"));
	}
	else if (!LoginURLDetails.IsValid())
	{
		ErrorStr = FString::Printf(TEXT("OnlineSubsystemFacebook is improperly configured in DefaultEngine.ini LoginURL=%s LoginRedirectUrl=%s ClientId=%s"),
			*LoginURLDetails.LoginUrl, *LoginURLDetails.LoginRedirectUrl, *LoginURLDetails.ClientId);
	}
	else
	{
		if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
		{
			ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
		}
		else
		{
			if (!AccountCredentials.Id.IsEmpty() && !AccountCredentials.Token.IsEmpty() && AccountCredentials.Type == GetAuthType())
			{
				bHasLoginOutstanding = true;

				Login(LocalUserNum, AccountCredentials.Token, FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineIdentityFacebook::OnAccessTokenLoginComplete));
			}
			else
			{
				IOnlineExternalUIPtr OnlineExternalUI = FacebookSubsystem->GetExternalUIInterface();
				if (OnlineExternalUI.IsValid())
				{
					LoginURLDetails.GenerateNonce();

					bHasLoginOutstanding = true;

					FOnLoginUIClosedDelegate CompletionDelegate = FOnLoginUIClosedDelegate::CreateRaw(this, &FOnlineIdentityFacebook::OnExternalUILoginComplete);
					OnlineExternalUI->ShowLoginUI(LocalUserNum, true, true, CompletionDelegate);
				}
				else
				{
					ErrorStr = FString::Printf(TEXT("External interface missing"));
				}
			}
		}
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("RegisterUser() failed: %s"),
			*ErrorStr);

		bHasLoginOutstanding = false;
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
		return false;
	}
	return true;
}

void FOnlineIdentityFacebook::Login(int32 LocalUserNum, const FString& AccessToken, const FOnLoginCompleteDelegate& InCompletionDelegate)
{
	FOnProfileRequestComplete CompletionDelegate = FOnProfileRequestComplete::CreateLambda([this, InCompletionDelegate](int32 LocalUserNumFromRequest, bool bWasSuccessful, const FString& ErrorStr)
	{
		FOnRequestCurrentPermissionsComplete NextCompletionDelegate = FOnRequestCurrentPermissionsComplete::CreateLambda([this, InCompletionDelegate](int32 LocalUserNumFromPerms, bool bWasSuccessful, const TArray<FSharingPermission>& Permissions)
		{
			OnRequestCurrentPermissionsComplete(LocalUserNumFromPerms, bWasSuccessful, Permissions, InCompletionDelegate);
		});

		if (bWasSuccessful)
		{
			RequestCurrentPermissions(LocalUserNumFromRequest, NextCompletionDelegate);
		}
		else
		{
			InCompletionDelegate.ExecuteIfBound(LocalUserNumFromRequest, bWasSuccessful, *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
		}
	});

	ProfileRequest(LocalUserNum, AccessToken, ProfileFields, CompletionDelegate);
}

void FOnlineIdentityFacebook::OnRequestCurrentPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& NewPermissions, FOnLoginCompleteDelegate CompletionDelegate)
{
	FString ErrorStr;
	if (!bWasSuccessful)
	{
		ErrorStr = TEXT("Failure to request current sharing permissions");
	}

	LoginURLDetails.ScopeFields.Empty(NewPermissions.Num());
	for (const FSharingPermission& Perm : NewPermissions)
	{
		if (Perm.Status == EOnlineSharingPermissionState::Granted)
		{
			LoginURLDetails.ScopeFields.Add(Perm.Name);
		}
	}

	LoginURLDetails.NewScopeFields.Empty();
	LoginURLDetails.RerequestScopeFields.Empty();

	CompletionDelegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, *GetUniquePlayerId(LocalUserNum), ErrorStr);
}

void FOnlineIdentityFacebook::OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UniqueId, const FString& Error)
{
	bHasLoginOutstanding = false;

	TriggerOnLoginCompleteDelegates(LocalUserNum, bWasSuccessful, UniqueId, Error);
	if (bWasSuccessful)
	{
		// login status changed
		TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, UniqueId);
	}
}

void FOnlineIdentityFacebook::OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error)
{
	const FString& ErrorStr = Error.GetErrorCode();
	const bool bWasSuccessful = Error.WasSuccessful() && UniqueId.IsValid() && UniqueId->IsValid();
	OnAccessTokenLoginComplete(ControllerIndex, bWasSuccessful, bWasSuccessful ? *UniqueId : *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
}

void FOnlineIdentityFacebook::RequestElevatedPermissions(int32 LocalUserNum, const TArray<FSharingPermission>& AddlPermissions, const FOnLoginCompleteDelegate& InCompletionDelegate)
{
	FString ErrorStr;

	if (bHasLoginOutstanding)
	{
		ErrorStr = FString::Printf(TEXT("Registration already pending for user"));
	}
	else if (!LoginURLDetails.IsValid())
	{
		ErrorStr = FString::Printf(TEXT("OnlineSubsystemFacebook is improperly configured in DefaultEngine.ini LoginURL=%s LoginRedirectUrl=%s ClientId=%s"),
			*LoginURLDetails.LoginUrl, *LoginURLDetails.LoginRedirectUrl, *LoginURLDetails.ClientId);
	}
	else
	{
		if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
		{
			ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
		}
		else
		{
			IOnlineExternalUIPtr OnlineExternalUI = FacebookSubsystem->GetExternalUIInterface();
			if (OnlineExternalUI.IsValid())
			{
				LoginURLDetails.GenerateNonce();

				TArray<FString> NewPerms;
				TArray<FString> RerequestPerms;
				for (const FSharingPermission& NewPermission : AddlPermissions)
				{
					if (!LoginURLDetails.ScopeFields.Contains(NewPermission.Name))
					{
						if (NewPermission.Status == EOnlineSharingPermissionState::Declined)
						{
							RerequestPerms.AddUnique(NewPermission.Name);
						}
						else
						{
							NewPerms.AddUnique(NewPermission.Name);
						}
					}
				}

				if (NewPerms.Num() > 0 || RerequestPerms.Num() > 0)
				{
					bHasLoginOutstanding = true;
					LoginURLDetails.NewScopeFields = NewPerms;
					LoginURLDetails.RerequestScopeFields = RerequestPerms;
					FOnLoginUIClosedDelegate CompletionDelegate = FOnLoginUIClosedDelegate::CreateRaw(this, &FOnlineIdentityFacebook::OnExternalUIElevatedPermissionsComplete, InCompletionDelegate);
					OnlineExternalUI->ShowLoginUI(LocalUserNum, true, true, CompletionDelegate);
				}
				else
				{
					// Fire off delegate now because permissions already exist
					FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
					InCompletionDelegate.ExecuteIfBound(LocalUserNum, true, *UserId, ErrorStr);
				}
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("External interface missing"));
			}
		}
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Error, TEXT("RequestElevatedPermissions() failed: %s"), *ErrorStr);
		bHasLoginOutstanding = false;
		InCompletionDelegate.ExecuteIfBound(LocalUserNum, false, *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
	}
}

void FOnlineIdentityFacebook::OnExternalUIElevatedPermissionsComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error, FOnLoginCompleteDelegate InCompletionDelegate)
{
	FString ErrorStr;
	bool bWasSuccessful = UniqueId.IsValid() && UniqueId->IsValid();
	bHasLoginOutstanding = false;

	if (!bWasSuccessful)
	{
		ErrorStr = TEXT("com.epicgames.elevated_perms_failed");
	}

	UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("RequestElevatedPermissions() %s"), bWasSuccessful ? TEXT("success") : TEXT("failed"));
	FUniqueNetIdPtr ExistingUserId = GetUniquePlayerId(ControllerIndex);
	InCompletionDelegate.ExecuteIfBound(ControllerIndex, bWasSuccessful, ExistingUserId.IsValid() ? *ExistingUserId : *FUniqueNetIdFacebook::EmptyId(), ErrorStr);
}

bool FOnlineIdentityFacebook::Logout(int32 LocalUserNum)
{
	bool bResult = false;
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		// remove cached user account
		UserAccounts.Remove(UserId->ToString());
		// remove cached user id
		UserIds.Remove(LocalUserNum);
		// reset scope permissions
		GConfig->GetArray(TEXT("OnlineSubsystemFacebook.OnlineIdentityFacebook"), TEXT("ScopeFields"), LoginURLDetails.ScopeFields, GEngineIni);

		TriggerOnLoginFlowLogoutDelegates(LoginDomains);

		// not async but should call completion delegate anyway
		FacebookSubsystem->ExecuteNextTick([this, LocalUserNum, UserId]() {
			TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
			TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *UserId);
		});

		bResult = true;
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		FacebookSubsystem->ExecuteNextTick([this, LocalUserNum]() {
			TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
		});
	}
	
	return bResult;
}

#endif // USES_RESTFUL_FACEBOOK
