// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_FACEBOOK
 
#include "OnlineIdentityFacebookCommon.h"
#include "OnlineSharingFacebookRest.h"
#include "OnlineAccountFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/** Windows implementation of a Facebook user account */
using FUserOnlineAccountFacebook  = FUserOnlineAccountFacebookCommon;

/**
 * Contains URL details for Windows Facebook interaction
 */
struct FFacebookLoginURL
{
	/** The endpoint at Facebook we are supposed to hit for auth */
	FString LoginUrl;
	/** The redirect url for Facebook to redirect to upon completion. Note: this is configured at Facebook too */
	FString LoginRedirectUrl;
	/** The client id given to us by Facebook */
	FString ClientId;
	/** Config based list of permission scopes to use when logging in */
	TArray<FString> ScopeFields;
	/** Requested list of permission scopes when elevating permissions (merge into ScopeFields on success) */
	TArray<FString> NewScopeFields;
	/** Requested list of permission scopes when elevating permissions (merge into ScopeFields on success) */
	TArray<FString> RerequestScopeFields;
	/** A value used to verify our response came from our server */
	FString State;
	/** Should the URL include the popup display size */
	bool bUsePopup;

	FFacebookLoginURL() 
		: bUsePopup(false)
	{ }

	bool IsValid() const
	{
		return !LoginUrl.IsEmpty() && !LoginRedirectUrl.IsEmpty() && !ClientId.IsEmpty();
	}

	FString GenerateNonce()
	{
		// random number to represent client generated state for verification on login
		State = FString::FromInt(FMath::Rand() % 100000);
		return State;
	}

	FString GetURL() const
	{
		FString Scopes = FString::Join(ScopeFields, TEXT(","));
		if (NewScopeFields.Num() > 0)
		{
			const FString AddlScopes = FString::Join(NewScopeFields, TEXT(","));
			Scopes += FString::Printf(TEXT(",%s"), *AddlScopes);
		}

		if (RerequestScopeFields.Num() > 0)
		{
			const FString AddlScopes = FString::Join(RerequestScopeFields, TEXT(","));
			Scopes += FString::Printf(TEXT(",%s&auth_type=rerequest"), *AddlScopes);
		}
		// &return_scopes=true
		// auth url to spawn in browser
		return FString::Printf(TEXT("%s?redirect_uri=%s&client_id=%s&state=%s&response_type=token&scope=%s%s"),
		*LoginUrl, *LoginRedirectUrl, *ClientId, *State, *Scopes, bUsePopup ? TEXT("&display=popup") : TEXT(""));
	}
};

/**
 * Facebook service implementation of the online identity interface
 */
class FOnlineIdentityFacebook :
	public FOnlineIdentityFacebookCommon
{
	/** Const details about communicating with Facebook API */
	FFacebookLoginURL LoginURLDetails;
	/** Whether we have a registration in flight or not */
	bool bHasLoginOutstanding;

	/** Domains used for login, for cookie management */
	TArray<FString> LoginDomains;

public:
	// IOnlineIdentity
	
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;

	// FOnlineIdentityFacebook

	FOnlineIdentityFacebook(FOnlineSubsystemFacebook* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityFacebook()
	{
	}

PACKAGE_SCOPE:

	/**
	 * Login with an existing access token
	 *
	 * @param LocalUserNum id of the local user initiating the request
	 * @param AccessToken access token already received from Facebook
	 * @param InCompletionDelegate delegate to fire when operation completes
	 */
	void Login(int32 LocalUserNum, const FString& AccessToken, const FOnLoginCompleteDelegate& InCompletionDelegate);

	/**
	 * Request additional permissions for an already logged in user.  Activates the external UI for user interaction
	 * 
	 * @param LocalUserNum id of the local user initiating the request
	 * @param AddlPermissions list of additional permissions that are been requested
	 * @param InCompletionDelegate delegate to fire when request has completed
	 */
	void RequestElevatedPermissions(int32 LocalUserNum, const TArray<FSharingPermission>& AddlPermissions, const FOnLoginCompleteDelegate& InCompletionDelegate);

	/** @return the login configuration details */
	const FFacebookLoginURL& GetLoginURLDetails() const { return LoginURLDetails; }

private:

	/**
	 * Delegate fired when the internal call to Login() with AccessToken is specified
	 *
	 * @param LocalUserNum index of the local user initiating the request
	 * @param bWasSuccessful was the login call successful
	 * @param UserId user id of the logged in user, or null if login failed
	 * @param Error error string if applicable
	 */
	void OnAccessTokenLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);

	/**
	 * Delegate fired when the call to ShowLoginUI completes 
	 */
	void OnExternalUILoginComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error);

	/**
	 * Delegate fired when the call to ShowLoginUI completes for requesting elevated permissions
	 */
	void OnExternalUIElevatedPermissionsComplete(FUniqueNetIdPtr UniqueId, const int ControllerIndex, const FOnlineError& Error, FOnLoginCompleteDelegate InCompletionDelegate);

	/**
	 * Delegate called when current permission request completes
	 *
	 * @param LocalUserNum user that made the request
	 * @param bWasSuccessful was the request successful
	 * @param NewPermissions array of all known permissions
	 * @param CompletionDelegate follow up delegate after this request is complete
	 */
	void OnRequestCurrentPermissionsComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FSharingPermission>& NewPermissions, FOnLoginCompleteDelegate CompletionDelegate);

};

typedef TSharedPtr<FOnlineIdentityFacebook, ESPMode::ThreadSafe> FOnlineIdentityFacebookPtr;

#endif // USES_RESTFUL_FACEBOOK
