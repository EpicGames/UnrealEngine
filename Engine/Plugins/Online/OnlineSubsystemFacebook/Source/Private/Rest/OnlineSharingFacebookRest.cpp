// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineSharingFacebookRest.h"
#include "OnlineIdentityFacebookRest.h"
#include "OnlineSubsystemFacebookPrivate.h"

FOnlineSharingFacebook::FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineSharingFacebookCommon(InSubsystem)
{
}

FOnlineSharingFacebook::~FOnlineSharingFacebook()
{
}

bool FOnlineSharingFacebook::RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions)
{
	bool bTriggeredRequest = false;

	ensure((NewPermissions & ~EOnlineSharingCategory::ReadPermissionMask) == EOnlineSharingCategory::None);

	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(LocalUserNum) == ELoginStatus::LoggedIn)
	{
		bTriggeredRequest = true;

		TArray<FSharingPermission> PermissionsNeeded;
		const bool bHasPermission = CurrentPermissions.HasPermission(NewPermissions, PermissionsNeeded);
		if (!bHasPermission)
		{
			FOnlineIdentityFacebookPtr IdentityIntFB = StaticCastSharedPtr<FOnlineIdentityFacebook>(IdentityInt);
			if (IdentityIntFB.IsValid())
			{
				IdentityIntFB->RequestElevatedPermissions(LocalUserNum, PermissionsNeeded, FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineSharingFacebook::OnPermissionsLevelRequest));
			}
		}
		else
		{
			// All permissions were already granted, no need to reauthorize
			TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, true);
		}
	}
	else
	{
		// If we weren't logged into Facebook we cannot do this action
		TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, false);
	}

	// We did kick off a request
	return bTriggeredRequest;
}

void FOnlineSharingFacebook::OnPermissionsLevelRequest(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error)
{
	UE_LOG_ONLINE_SHARING(Display, TEXT("OnPermissionsLevelRequest : Success - %d %s"), bWasSuccessful, *Error);
	TriggerOnRequestNewReadPermissionsCompleteDelegates(LocalUserNum, bWasSuccessful);
}

bool FOnlineSharingFacebook::RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy)
{
	/** NYI */
	ensure((NewPermissions & ~EOnlineSharingCategory::PublishPermissionMask) == EOnlineSharingCategory::None);

	bool bTriggeredRequest = false;
	TriggerOnRequestNewPublishPermissionsCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

bool FOnlineSharingFacebook::ShareStatusUpdate(int32 LocalUserNum, const FOnlineStatusUpdate& StatusUpdate)
{
	/** NYI */
	bool bTriggeredRequest = false;
	TriggerOnSharePostCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

bool FOnlineSharingFacebook::ReadNewsFeed(int32 LocalUserNum, int32 NumPostsToRead)
{
	/** NYI */
	bool bTriggeredRequest = false;
	TriggerOnReadNewsFeedCompleteDelegates(LocalUserNum, false);
	return bTriggeredRequest;
}

#endif // USES_RESTFUL_FACEBOOK
