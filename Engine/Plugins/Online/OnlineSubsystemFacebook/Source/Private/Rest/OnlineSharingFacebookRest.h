// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USES_RESTFUL_FACEBOOK
 
// Module includes
#include "OnlineSharingFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

class FOnlineSubsystemFacebook;

/**
 * Facebook implementation of the Online Sharing Interface
 */
class FOnlineSharingFacebook : public FOnlineSharingFacebookCommon
{

public:

	//~ Begin IOnlineSharing Interface
	virtual bool ReadNewsFeed(int32 LocalUserNum, int32 NumPostsToRead) override;
	virtual bool RequestNewReadPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions) override;
	virtual bool ShareStatusUpdate(int32 LocalUserNum, const FOnlineStatusUpdate& StatusUpdate) override;
	virtual bool RequestNewPublishPermissions(int32 LocalUserNum, EOnlineSharingCategory NewPermissions, EOnlineStatusUpdatePrivacy Privacy) override;
	//~ End IOnlineSharing Interface

public:

	/**
	 * Constructor used to indicate which OSS we are a part of
	 */
	FOnlineSharingFacebook(FOnlineSubsystemFacebook* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineSharingFacebook();

private:

	void OnPermissionsLevelRequest(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error);
};

typedef TSharedPtr<FOnlineSharingFacebook, ESPMode::ThreadSafe> FOnlineSharingFacebookPtr;

#endif // USES_RESTFUL_FACEBOOK