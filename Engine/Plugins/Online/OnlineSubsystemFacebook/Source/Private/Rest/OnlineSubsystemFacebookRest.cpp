// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineSubsystemFacebook.h"
#include "OnlineSubsystemFacebookPrivate.h"
#include "OnlineIdentityFacebookRest.h"
#include "OnlineFriendsFacebookRest.h"
#include "OnlineSharingFacebookRest.h"
#include "OnlineExternalUIInterfaceFacebookRest.h"

FOnlineSubsystemFacebook::FOnlineSubsystemFacebook(FName InInstanceName)
	: FOnlineSubsystemFacebookCommon(InInstanceName)
{
}

FOnlineSubsystemFacebook::~FOnlineSubsystemFacebook()
{
}

bool FOnlineSubsystemFacebook::Init()
{
	if (FOnlineSubsystemFacebookCommon::Init())
	{
		FacebookIdentity = MakeShareable(new FOnlineIdentityFacebook(this));
		FacebookFriends = MakeShareable(new FOnlineFriendsFacebook(this));
		FacebookExternalUI = MakeShareable(new FOnlineExternalUIFacebook(this));
		FacebookSharing = MakeShareable(new FOnlineSharingFacebook(this));
		return true;
	}

	return false;
}

bool FOnlineSubsystemFacebook::Shutdown()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemFacebook::Shutdown()"));
	return FOnlineSubsystemFacebookCommon::Shutdown();
}

bool FOnlineSubsystemFacebook::IsEnabled() const
{
	// Overridden due to different platform implementations of IsEnabled
	return FOnlineSubsystemFacebookCommon::IsEnabled();
}

#endif // USES_RESTFUL_FACEBOOK
