// Copyright Epic Games, Inc. All Rights Reserved.

#if USES_RESTFUL_FACEBOOK

#include "OnlineUserFacebookRest.h"
#include "OnlineSubsystemFacebookPrivate.h"

FOnlineUserFacebook::FOnlineUserFacebook(FOnlineSubsystemFacebook* InSubsystem)
	: FOnlineUserFacebookCommon(InSubsystem)
{
}

FOnlineUserFacebook::~FOnlineUserFacebook()
{
}

#endif // USES_RESTFUL_FACEBOOK
