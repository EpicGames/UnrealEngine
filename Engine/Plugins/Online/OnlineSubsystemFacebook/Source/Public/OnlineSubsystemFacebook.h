// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemFacebookCommon.h"
#include "OnlineSubsystemFacebookPackage.h"

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityFacebook, ESPMode::ThreadSafe> FOnlineIdentityFacebookPtr;
typedef TSharedPtr<class FOnlineFriendsFacebook, ESPMode::ThreadSafe> FOnlineFriendsFacebookPtr;
typedef TSharedPtr<class FOnlineSharingFacebook, ESPMode::ThreadSafe> FOnlineSharingFacebookPtr;
typedef TSharedPtr<class FOnlineUserFacebook, ESPMode::ThreadSafe> FOnlineUserFacebookPtr;
typedef TSharedPtr<class FOnlineExternalUIFacebook, ESPMode::ThreadSafe> FOnlineExternalUIFacebookPtr;

/**
 *	OnlineSubsystemFacebook - Implementation of the online subsystem for Facebook services
 */
class ONLINESUBSYSTEMFACEBOOK_API FOnlineSubsystemFacebook
	: public FOnlineSubsystemFacebookCommon
{
public:

	// FOnlineSubsystemFacebookCommon Interface
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual bool IsEnabled() const override;

	// FOnlineSubsystemFacebook

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemFacebook();

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemFacebook() = delete;
	explicit FOnlineSubsystemFacebook(FName InInstanceName);
};

typedef TSharedPtr<FOnlineSubsystemFacebook, ESPMode::ThreadSafe> FOnlineSubsystemFacebookPtr;
