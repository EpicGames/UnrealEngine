// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "OnlineSubsystemFacebookPackage.h"

/**
 * Online subsystem module class (Facebook Implementation)
 * Code related to the loading and handling of the Facebook module.
 */
class FOnlineSubsystemFacebookModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryFacebook* FacebookFactory;

public:	

	// IModuleInterface 

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}

	// FOnlineSubsystemFacebookModule

	/**
	 * Constructor
	 */
	FOnlineSubsystemFacebookModule() :
		FacebookFactory(NULL)
	{}

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemFacebookModule() 
	{}
};


