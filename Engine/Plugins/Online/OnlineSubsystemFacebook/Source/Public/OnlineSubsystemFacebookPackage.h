// Copyright Epic Games, Inc. All Rights Reserved.

// Can't be #pragma once because other modules may define PACKAGE_SCOPE

// Intended to be the last include in an exported class definition
// Properly defines some members as "public to the module" vs "private to the consumer/user"

#undef PACKAGE_SCOPE
#ifdef ONLINESUBSYSTEMFACEBOOK_PACKAGE
#define PACKAGE_SCOPE public
#else
#define PACKAGE_SCOPE protected
#endif
